{
	"FileVersion": 3,
	"bIsPluginExtension": true,
	"Modules": [
		{
			"Name": "OpenExrWrapper",
			"Type": "RuntimeNoCommandlet",
			"PlatformAllowList": [
				"WinGDK"
			]
		},
		{
			"Name": "ExrReaderGpu",
			"Type": "Runtime",
			"PlatformAllowList": [
				"WinGDK"
			]
		}
	]
}