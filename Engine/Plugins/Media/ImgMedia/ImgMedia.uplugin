{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Image Sequence Media Player",
	"Description" : "Implements a media player for image sequences in EXR and other formats.",
	"Category" : "Media Players",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "https://forums.unrealengine.com/showthread.php?46879-Media-Framework-Documentation-for-4-5-Preview",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "ImgMedia",
			"Type" : "RuntimeNoCommandlet",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "ImgMediaEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		},
		{
			"Name": "ImgMediaEngine",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name" : "ImgMediaFactory",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		},
		{
			"Name" : "ImgMediaFactory",
			"Type" : "RuntimeNoCommandlet",
			"LoadingPhase" : "PostEngineInit"
		},
		{
			"Name" : "OpenExrWrapper",
			"Type" : "RuntimeNoCommandlet",
			"LoadingPhase" : "PostEngineInit",
			"PlatformAllowList" : [ "Mac", "Win64" ]
		},
		{
			"Name" : "ExrReaderGpu",
			"Type" : "Runtime",
			"LoadingPhase" : "PostConfigInit",
			"PlatformAllowList" : [ "Win64" ]
		}
	],
	"Plugins":
	[
		{
			"Name": "TmvMedia",
			"Enabled": true
		},
		{
			"Name": "MediaPlayerEditor",
			"TargetAllowList": [ "Editor" ],
			"Enabled": true
		},
	]
}
