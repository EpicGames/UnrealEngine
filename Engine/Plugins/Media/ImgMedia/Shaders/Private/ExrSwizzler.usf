// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/GammaCorrectionCommon.ush"

#define NUM_CHANNELS (PERMUTATION_CHANNELS + 1)

#if PARTIAL_TILES
struct FTileDesc
{
	int2 TileResolution;
	uint BufferOffset;
	int AlignPadding;
};
#endif

/** The unswizzled buffer packed into 4 byte buffer stride. */
StructuredBuffer<uint> UnswizzledBuffer;

#if PARTIAL_TILES
/** The unswizzled buffer packed into 4 byte buffer stride. */
StructuredBuffer<FTileDesc> TileDescBuffer;
#endif

/** Texture size that the buffer is supposed to be unpacked to. */
int2 TextureSize;

/** Size of tiles that represent this texture. */
int2 TileSize;

/** Total number of tiles in X and Y direction for the current mip level. */
int2 NumTiles;

/** Number of channels per pixel. */
int NumChannels;

/** Flag to enable color space transformations. */
uint bApplyColorTransform;

/** Source encoding to linearize. */
uint EOTF;

/** Color space transformation matrix, from source to working. */
float4x4 ColorSpaceMatrix;

// Same definitions as MediaShaders.usf
#define UE_Color_EEncoding_Linear		1u
#define UE_Color_EEncoding_sRGB			2u
#define UE_Color_EEncoding_ST2084		3u
#define UE_Color_EEncoding_SLog3		12u
#define UE_Color_EEncoding_HLG				17u

float3 ApplyColorTransform(float3 Color, float3x3 InColorSpaceMatrix, uint InEOTF)
{
	// Linearize
	switch(InEOTF)
	{
	case UE_Color_EEncoding_sRGB:			Color = sRGBToLinear(Color); break;
	case UE_Color_EEncoding_ST2084:			Color = ST2084ToLinear(Color); break;
	case UE_Color_EEncoding_SLog3:			Color = SLog3ToLinear(Color); break;
	case UE_Color_EEncoding_HLG:			Color = HLGToLinear(Color); break;
	default: break;
	}

	// Adjust colorspace
	return mul(InColorSpaceMatrix, Color);
}

/** Unpacks data from the provided structured buffer. */
half GetBufferValue(int Offset, int2 TextureSize, int ChannelIndex)
{
	// the location as if it would've been if the buffer was unpacked.
	int UnpackedLocation = Offset + TextureSize.x * ChannelIndex;
	// Byte offset within the 4 byte packed value.
	int Remainder = UnpackedLocation % 2;

	// Position in a packed buffer.
	int PackedPosition = (UnpackedLocation - Remainder) >> 1;

	uint PackedValue = (UnswizzledBuffer[PackedPosition] >> (16 * Remainder)) & 0xffff;

	// Convert from uint32 to f16
	return (half)f16tof32(PackedValue);
}


/** Vertex Shader. */
void MainVS(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InTexCoord : ATTRIBUTE1,
	out noperspective float4 OutUVAndScreenPos : TEXCOORD0,
	out float4 OutPosition : SV_POSITION)
{
	DrawRectangle(InPosition, InTexCoord, OutPosition, OutUVAndScreenPos);
}

#if PARTIAL_TILES
void GetStartScanlinePosition_PartialTiles(in int2 PixelPos, out int2 SampleSize, out int StartPosition)
{
	int2 TileCoord = int2(ceil(PixelPos.x / TileSize.x), ceil(PixelPos.y / TileSize.y));
	FTileDesc TileDesc = TileDescBuffer[TileCoord.y * NumTiles.x + TileCoord.x];

	SampleSize = TileDesc.TileResolution;

	int xCoord = (PixelPos.x) % TileSize.x;
	int yCoord = (PixelPos.y) % TileSize.y;

	// Padding in buffer elements (uint)
	const int TilePadding = 10;

	// Offset is in bytes while our buffer is a packed uint16 into uint32. Therefore divide by 2. 
	int PreviousTilesOffset = TileDesc.BufferOffset / 2;

	// Current tile position.
	StartPosition = PreviousTilesOffset + yCoord * TileDesc.TileResolution.x * NumChannels + xCoord + TilePadding;
}
#endif

void GetStartScanlinePosition(in int2 PixelPos, out int2 SampleSize, out int StartPosition)
{
	// Depending on the EXR file it could contain multi-channel data.
#if RENDER_TILES
	SampleSize = TileSize;

	int xCoord = (PixelPos.x) % TileSize.x;
	int yCoord = (PixelPos.y) % TileSize.y;

	int TileX = floor((PixelPos.x) / TileSize.x);
	int TileY = floor((PixelPos.y) / TileSize.y);

	// Padding in buffer elements (uint)
	const int TilePadding = 10;

	// Current tile position.
	int TileBufferStride = TileSize.x * TileSize.y * NumChannels;
	int PreviousTilesOffset = (NumTiles.x * TileY + TileX) * TileBufferStride;
	StartPosition = PreviousTilesOffset + yCoord * TileSize.x * NumChannels + xCoord + (TileY * NumTiles.x + TileX + 1) * TilePadding;

#else
	SampleSize.x = TextureSize.x;
	SampleSize.y = TextureSize.y;

	// Padding in bytes
	const int TilePadding = 4;

	// Current scanline position.
	StartPosition = PixelPos.y * TextureSize.x * NumChannels + PixelPos.x + (PixelPos.y + 1) * 4;
#endif //RENDER_TILES
}

/** Pixel Shader. */
half4 MainPS(noperspective float4 UVAndScreenPos : TEXCOORD0) : SV_Target0
{

	float2 UV = UVAndScreenPos.xy;
	
	int2 SampleSize;

	int2 PixelPos = UV * TextureSize;

	int StartBufferPosition;

#if PARTIAL_TILES
	GetStartScanlinePosition_PartialTiles(PixelPos, SampleSize, StartBufferPosition);
#else
	GetStartScanlinePosition(PixelPos, SampleSize, StartBufferPosition);
#endif

	int ChannelIndex = 0;
	half4 Color = half4(0.0f, 0.0f, 0.0f, 1.0f);

#if NUM_CHANNELS == 4
	Color.a = GetBufferValue(StartBufferPosition, SampleSize, ChannelIndex++);
#endif
#if NUM_CHANNELS >= 3
	Color.b = GetBufferValue(StartBufferPosition, SampleSize, ChannelIndex++);
#endif
#if NUM_CHANNELS >= 2
	Color.g = GetBufferValue(StartBufferPosition, SampleSize, ChannelIndex++);
#endif
#if NUM_CHANNELS >= 1
	Color.r = GetBufferValue(StartBufferPosition, SampleSize, ChannelIndex++);
#endif

	// Special case for single channel EXRs: we convert to grayscale by duplicating the red channel.
#if NUM_CHANNELS == 1
	Color.rgb = Color.r;
#endif

	if(bApplyColorTransform)
	{
		Color.rgb = ApplyColorTransform(Color.rgb, (float3x3)ColorSpaceMatrix, EOTF);
	}

	return Color;
}