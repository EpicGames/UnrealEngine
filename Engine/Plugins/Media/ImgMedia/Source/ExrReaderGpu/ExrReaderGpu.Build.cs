// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ExrReaderGpu : ModuleRules
	{
		public ExrReaderGpu(ReadOnlyTargetRules Target) : base(Target)
		{
			bEnableExceptions = true;
			bDisableAutoRTFMInstrumentation = true; // AutoRTFM cannot be used with exceptions
			bUseRTTI = true;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"RenderCore",
					"Projects",
				});

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				AddEngineThirdPartyPrivateStaticDependencies(Target, "Imath");
				AddEngineThirdPartyPrivateStaticDependencies(Target, "UEOpenExr");
				PrivateDependencyModuleNames.Add("OpenExrWrapper");
			}
		}
	}
}
