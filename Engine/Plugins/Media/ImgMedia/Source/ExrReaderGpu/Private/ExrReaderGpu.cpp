// Copyright Epic Games, Inc. All Rights Reserved.
#include "ExrReaderGpu.h"

#if PLATFORM_WINDOWS

#include "ExrReaderGpuModule.h"
#include "HAL/IConsoleManager.h"

PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
	#include "Imath/ImathBox.h"
	#include "OpenEXR/ImfCompressionAttribute.h"
	#include "OpenEXR/ImfHeader.h"
	#include "OpenEXR/ImfRgbaFile.h"
	#include "OpenEXR/ImfStandardAttributes.h"
	#include "OpenEXR/ImfVersion.h"
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END

static TAutoConsoleVariable<bool> CVarForceTileDescBufferExrGpuReader(
	TEXT("r.ExrReaderGPU.ForceTileDescBuffer"),
	true,
	TEXT("Calculates tile description and offsets on CPU and provides a Structured buffer.\n")
	TEXT("to be used to access tile description on GPU\n"));

namespace
{
	bool ReadBytes(FILE* FileHandle, char* Buffer, int32 Length)
	{
		int32 Result = -1;
		try
		{
			Result = fread(Buffer, 1, Length, FileHandle);
		}
		catch (...)
		{
			UE_LOGF(LogExrReaderGpu, Error, "Error reading bytes");
			return false;
		}
		
		if (Result != Length)
		{
			UE_LOGF(LogExrReaderGpu, Error, "Error reading file %i %i", ferror(FileHandle), feof(FileHandle));
			return false;
		}
		return true;
	}

	template <int32 Length>
	bool CheckIsNullTerminated(const char(&StringToCheck)[Length])
	{
		for (int32 i = 0; i < Length; ++i) 
		{
			if (StringToCheck[i] == '\0')
			{
				return true;
			}
		}

		// The string was never terminated.
		UE_LOGF(LogExrReaderGpu, Error, "Invalid EXR file with string that was never terminated");
		return false;
	}

	bool Read4ByteValue(FILE* In, int32& OutValue)
	{
		char ThrowAwayValue[4];
		if (!ReadBytes(In, ThrowAwayValue, 4))
		{
			return false;
		}

		OutValue = (static_cast <unsigned char> (ThrowAwayValue[0]) & 0x000000ff) |
			((static_cast <unsigned char> (ThrowAwayValue[1]) << 8) & 0x0000ff00) |
			((static_cast <unsigned char> (ThrowAwayValue[2]) << 16) & 0x00ff0000) |
			(static_cast <unsigned char> (ThrowAwayValue[3]) << 24);
		return true;
	}


	bool Read8ByteValue(FILE* InFileHandle, int64& OutValue)
	{
		unsigned char ThrowAwayValue[8];
		if (!ReadBytes(InFileHandle, reinterpret_cast<char*>(ThrowAwayValue), 8))
		{
			return false;
		}

		OutValue = ((uint64)ThrowAwayValue[0] & 0x00000000000000ffLL) |
			(((uint64)ThrowAwayValue[1] << 8) & 0x000000000000ff00LL) |
			(((uint64)ThrowAwayValue[2] << 16) & 0x0000000000ff0000LL) |
			(((uint64)ThrowAwayValue[3] << 24) & 0x00000000ff000000LL) |
			(((uint64)ThrowAwayValue[4] << 32) & 0x000000ff00000000LL) |
			(((uint64)ThrowAwayValue[5] << 40) & 0x0000ff0000000000LL) |
			(((uint64)ThrowAwayValue[6] << 48) & 0x00ff000000000000LL) |
			((uint64)ThrowAwayValue[7] << 56);
		return true;
	}

	bool ReadString(FILE* InFileHandle, int32 Length, char OutValue[])
	{
		while (Length >= 0)
		{
			if (!ReadBytes(InFileHandle, OutValue, 1))
			{
				return false;
			}

			if (*OutValue == 0)
			{
				break;
			}

			--Length;
			++OutValue;
		}
		return true;
	}

	bool ReadString(FILE* InFileHandle, int32 InSize)
	{
		TArray<char> Value;
		Value.SetNum(InSize);

		for (int32 CharNum = 0; CharNum < InSize; CharNum++)
		{
			if (!ReadBytes(InFileHandle, &Value[CharNum], 1))
			{
				return false;
			}
		}
		return true;
	}

}

bool FExrReader::ReadMagicNumberAndVersionField(FILE* FileHandle)
{
	using namespace OPENEXR_IMF_INTERNAL_NAMESPACE;

	int32 Magic;
	int32 Version;
	if (!Read4ByteValue(FileHandle, Magic))
	{
		return false;
	}
	Read4ByteValue(FileHandle, Version);

	if (Magic != MAGIC || getVersion(Version) != EXR_VERSION || !supportsFlags(getFlags(Version)))
	{
		UE_LOGF(LogExrReaderGpu, Error, "Invalid EXR file has been detected.");
		return false;
	}

	return true;
}

bool FExrReader::ReadHeaderData(FILE* FileHandle)
{
	int32 AttrCount = 0;
	bool bNothingIsRead = false;

	// Read all header attributes.
	while (true)
	{
		char AttributeName[STRING_SIZE];
		ReadString(FileHandle, MAX_LENGTH, AttributeName);

		// Check to make sure it is not null. And if it is - its the end of the header.
		if (AttributeName[0] == 0)
		{
			if (AttrCount == 0)
			{
				bNothingIsRead = true;
			}

			break;
		}

		AttrCount++;

		// Read the attribute type and the size of the attribute value.
		char TypeName[STRING_SIZE];
		int32 Size;

		ReadString(FileHandle, MAX_LENGTH, TypeName);
		CheckIsNullTerminated(TypeName);
		Read4ByteValue(FileHandle, Size);

		if (Size < 0)
		{
			UE_LOGF(LogExrReaderGpu, Error, "Invalid EXR file has been detected.");
			return false;
		}

		try
		{
			ReadString(FileHandle, Size);
		}
		catch (...)
		{
			UE_LOGF(LogExrReaderGpu, Error, "Issue reading attribute from EXR: %s", AttributeName);
			return false;
		}
	}
	return true;
}

bool FExrReader::ReadLineOrTileOffsets(FILE* FileHandle, ELineOrder LineOrder, TArray<int64>& LineOrTileOffsets)
{
	// At the moment we only support increasing Y EXR; 
	// LineOrder is currently unused but we might need to take that into account as exr scanlines can go from top to bottom and vice versa.
	if (LineOrder != INCREASING_Y)
	{
		UE_LOGF(LogExrReaderGpu, Error, "Unsupported Line order in EXR: %d", LineOrder);
		return false;
	}

	for (int32 i = 0; i < LineOrTileOffsets.Num(); i++)
	{
		Read8ByteValue(FileHandle, LineOrTileOffsets[i]);
	}
	return true;
}

bool FExrReader::GenerateTextureData(uint16* Buffer, int32 BufferSize, FString FilePath, int32 NumberOfScanlines, int32 NumChannels)
{
	check(Buffer != nullptr);

	FILE* FileHandle;
	errno_t Error = fopen_s(&FileHandle, TCHAR_TO_ANSI(*FilePath), "rb");

	if (FileHandle == NULL || Error != 0)
	{
		return false;
	}

	fseek(FileHandle, 0, SEEK_END);
	int64 FileLength = ftell(FileHandle);
	rewind(FileHandle);

	// Reading header (throwaway) and line offset data. We just need line offset data.
	TArray<int64> LineOrTileOffsets;
	{
		ReadMagicNumberAndVersionField(FileHandle);
		ReadHeaderData(FileHandle);
		LineOrTileOffsets.SetNum(NumberOfScanlines);

		// At the moment we support only increasing Y.
		ELineOrder LineOrder = INCREASING_Y;
		ReadLineOrTileOffsets(FileHandle, LineOrder, LineOrTileOffsets);
	}

	fseek(FileHandle, LineOrTileOffsets[0], SEEK_SET);
	fread(Buffer, BufferSize, 1 /*NumOfElements*/, FileHandle);

	fclose(FileHandle);
	return true;
}


void FExrReader::CalculateTileOffsets
	( TArray<int32>& OutNumTilesPerLevel
	, TArray<TArray<FTileDesc>>& OutPartialTileInfo
	, const FIntPoint& FullTextureResolution
	, const FIntPoint& TileDimWithBorders
	, int32 NumMipLevels
	, int64 PixelSize)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CalculateTileOffsets);

	int64 CurrentPosition = 0;
	for (int32 MipLevel = 0; MipLevel < NumMipLevels; MipLevel++)
	{
		int32 MipLevelDiv = 1 << MipLevel;

		// Resolution of the texture in pixels for this mip level.
		FIntPoint MipResolution = (FullTextureResolution / MipLevelDiv).ComponentMax(FIntPoint(1,1));

		// Dimension of the texture in tiles including partial tiles
		float TileResFractionX = float(MipResolution.X) / TileDimWithBorders.X;
		float TileResFractionY = float(MipResolution.Y) / TileDimWithBorders.Y;
		FIntPoint DimensionInTiles_PartialTiles
			( FMath::CeilToInt(TileResFractionX)
			, FMath::CeilToInt(TileResFractionY));

		// Dimension of the texture in tiles excluding partial tiles.
		FIntPoint DimensionInTiles_CompleteTiles
			( FMath::FloorToInt(TileResFractionX)
			, FMath::FloorToInt(TileResFractionY));

		// Total number of tiles for this mip level.
		int32 NumActualTiles = DimensionInTiles_PartialTiles.X * DimensionInTiles_PartialTiles.Y;

		OutNumTilesPerLevel.Add(DimensionInTiles_PartialTiles.X * DimensionInTiles_PartialTiles.Y);

		const bool bHasPartialTiles = (DimensionInTiles_PartialTiles != DimensionInTiles_CompleteTiles) || CVarForceTileDescBufferExrGpuReader.GetValueOnAnyThread();
		if (!bHasPartialTiles)
		{
			continue;
		}

		OutPartialTileInfo.Add({});
		TArray<FTileDesc>& TileInfoList = OutPartialTileInfo[MipLevel];
		TileInfoList.SetNum(NumActualTiles);

		// Resolution of the partial tile in the bottom right corner.
		const FIntPoint PartialTileResolution = FIntPoint(MipResolution.X % TileDimWithBorders.X, MipResolution.Y % TileDimWithBorders.Y);
		const int64 MipOffsetStart = CurrentPosition;

		for (int TileIndex = 0; TileIndex < NumActualTiles; TileIndex++)
		{
			FIntPoint TileDim(TileDimWithBorders);

			if (DimensionInTiles_PartialTiles.X != DimensionInTiles_CompleteTiles.X)
			{
				// If true - this is a partial tile in X dimension.
				if (((TileIndex + 1) % DimensionInTiles_PartialTiles.X) == 0)
				{
					TileDim.X = PartialTileResolution.X;
				}
			}

			if (DimensionInTiles_PartialTiles.Y != DimensionInTiles_CompleteTiles.Y)
			{
				// If true - this is a partial tile in Y dimension.
				if (TileIndex >= (DimensionInTiles_PartialTiles.X * DimensionInTiles_CompleteTiles.Y))
				{
					TileDim.Y = PartialTileResolution.Y;
				}
			}

			TileInfoList[TileIndex] = { TileDim, (uint32)(CurrentPosition - MipOffsetStart) };

			// Tile offset.
			CurrentPosition += TileDim.X * TileDim.Y * PixelSize;

			// Vanilla exr has 20 byte padding at the beginning of each tile.
			CurrentPosition += FExrReader::TILE_PADDING;
		}
	}
}

bool FExrReader::OpenExrAndPrepareForPixelReading(FString FilePath, const TArray<int32>& NumOffsetsPerLevel)
{
	if (FileHandle != nullptr)
	{
		UE_LOGF(LogExrReaderGpu, Error, "The file has already been open for reading but never closed.");
		return false;
	}

	errno_t Error = fopen_s(&FileHandle, TCHAR_TO_ANSI(*FilePath), "rb");

	if (FileHandle == NULL || Error != 0)
	{
		return false;
	}


	// Reading header (throwaway) and line offset data. We just need line offset data.
	{
		ReadMagicNumberAndVersionField(FileHandle);
		if (!ReadHeaderData(FileHandle))
		{
			return false;
		}

		LineOrTileOffsetsPerLevel.SetNum(NumOffsetsPerLevel.Num());

		// At the moment we support only increasing Y.
		ELineOrder LineOrder = INCREASING_Y;
		for (int Level = 0; Level < LineOrTileOffsetsPerLevel.Num(); Level++)
		{
			TArray<int64>& OffsetsPerLevel = LineOrTileOffsetsPerLevel[Level];
			OffsetsPerLevel.SetNum(NumOffsetsPerLevel[Level]);
			ReadLineOrTileOffsets(FileHandle, LineOrder, OffsetsPerLevel);
		}
	}

	fseek(FileHandle, 0, SEEK_END);
	FileLength = ftell(FileHandle);
	fseek(FileHandle, LineOrTileOffsetsPerLevel[0][0], SEEK_SET);

	return true;
}


bool FExrReader::ReadExrImageChunk(void* Buffer, int64 ChunkSize)
{
	if (FileHandle == nullptr)
	{
		UE_LOGF(LogExrReaderGpu, Error, "File is not open for reading. Please use OpenExrAndPrepareForPixelReading. ");
		return false;
	}
	if (Buffer == nullptr)
	{
		UE_LOGF(LogExrReaderGpu, Error, "Buffer provided is invalid. Please provide a valid buffer. ");
		return false;
	}
	size_t NumElements = 1;
	size_t NumElementsRead = fread(Buffer, ChunkSize, NumElements, FileHandle);
	if (NumElementsRead != NumElements)
	{
		UE_LOGF(LogExrReaderGpu, Error, "Issue reading EXR chunk. ");
	}
	return NumElementsRead == NumElements;
}

bool FExrReader::SeekTileWithinFile(const int32 StartTileIndex, const int32 Level, int64& OutBufferOffset)
{
	if (StartTileIndex >= LineOrTileOffsetsPerLevel[Level].Num())
	{
		UE_LOGF(LogExrReaderGpu, Error, "Tile index is invalid.");
		return false;
	}

	int64 LineOffset = LineOrTileOffsetsPerLevel[Level][StartTileIndex];
	OutBufferOffset = LineOffset - LineOrTileOffsetsPerLevel[Level][0];
	return fseek(FileHandle, LineOffset, SEEK_SET) == 0;
}

bool FExrReader::GetByteOffsetForTile(const int32 TileIndex, const int32 Level, int64& OutBufferOffset)
{
	if (LineOrTileOffsetsPerLevel.Num() <= Level || LineOrTileOffsetsPerLevel[Level].Num() < TileIndex)
	{
		return false;
	}
	else
	{
		// In some cases we'd like to know the position of the final byte of the last tile.
		if (TileIndex == LineOrTileOffsetsPerLevel[Level].Num())
		{
			// If this is the last mip and the last tile that means that we don't have 
			if (LineOrTileOffsetsPerLevel.Num() > Level + 1)
			{
				OutBufferOffset = LineOrTileOffsetsPerLevel[Level + 1][0];
			}
			else
			{
				OutBufferOffset = FileLength;
			}
		}
		else
		{
			OutBufferOffset = LineOrTileOffsetsPerLevel[Level][TileIndex];
		}
		return true;
	}
}

bool FExrReader::CloseExrFile()
{
	if (FileHandle == nullptr)
	{
		UE_LOGF(LogExrReaderGpu, Error, "File is not open for reading. Please use OpenExrAndPrepareForPixelReading");
		return false;
	}
	fclose(FileHandle);
	LineOrTileOffsetsPerLevel.Empty();
	return true;
}

#endif
