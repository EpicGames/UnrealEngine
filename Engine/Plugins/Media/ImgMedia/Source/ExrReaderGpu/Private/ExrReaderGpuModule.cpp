// Copyright Epic Games, Inc. All Rights Reserved.


#include "ExrReaderGpuModule.h"

#include "HAL/Platform.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "ShaderCore.h"

DEFINE_LOG_CATEGORY(LogExrReaderGpu);

#define LOCTEXT_NAMESPACE "FExrReaderGpuModule"

void FExrReaderGpuModule::StartupModule()
{
#if PLATFORM_WINDOWS
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("ImgMedia"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/ExrReaderShaders"), PluginShaderDir);
#endif
}

void FExrReaderGpuModule::ShutdownModule()
{

}
#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FExrReaderGpuModule, ExrReaderGpu);
