// Copyright Epic Games, Inc. All Rights Reserved.

#if defined(PLATFORM_WINDOWS) && PLATFORM_WINDOWS

#include "ExrSwizzlingShader.h"
#include "GlobalShader.h"
#include "ShaderCore.h"
#include "ShaderParameterMacros.h"
#include "CoreMinimal.h"

IMPLEMENT_GLOBAL_SHADER(FExrSwizzlePS, "/Plugin/ExrReaderShaders/Private/ExrSwizzler.usf", "MainPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FExrSwizzleVS, "/Plugin/ExrReaderShaders/Private/ExrSwizzler.usf", "MainVS", SF_Vertex);

#endif