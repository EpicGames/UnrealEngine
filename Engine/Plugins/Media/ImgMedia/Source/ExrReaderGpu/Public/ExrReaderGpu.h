// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CoreMinimal.h"

#if PLATFORM_WINDOWS
#include <cstdio>


/**
* This class is a collection of distant and compressed takes on 
* OpenExr's ScanLineInputFile and ReadPixel functionality.
*/
class FExrReader
{
public:

	struct FTileDesc
	{
		FIntPoint TileResolution;
		uint32 BufferOffset;

		// Optimization: align to 128 bit. 
		int32 AlignPadding;
	};

	/** 
	*	This is based on Open Exr implementation.
	*	We need to handle all of these in the future. At the moment only INCREASING_Y is 
	*	supported.
	*/
	enum ELineOrder
	{
		INCREASING_Y,
		DECREASING_Y,
		RANDOM_Y // Tiled files.
	};
private:
	static const int32	STRING_SIZE = 256;
	static const int32	MAX_LENGTH = STRING_SIZE - 1;

public:
	/*At the beginning of each row of B G R channel planes there is 2x4 byte data that has information*/
	static const int32	PLANAR_RGB_SCANLINE_PADDING = 8;
	/*At the beginning of each tile there is 20 byte data that has information*/
	static const int32	TILE_PADDING = 20;

public:
	FExrReader() 
		: FileHandle(nullptr)
		, FileLength(0)
		, LineOrTileOffsetsPerLevel()
	{};
private:

	static EXRREADERGPU_API bool ReadMagicNumberAndVersionField(FILE* FileHandle);

	/** 
	* This is just so we can get line offsets. Based on OpenExr implementation with some caveats.
	* The header in general is discarded, but without reading it - it is impossible to get to the actual pixel data reliably. 
	* Returns false if it has failed reading the file in any way.
	*/
	static EXRREADERGPU_API bool ReadHeaderData(FILE* FileHandle);

	/** Reads Scan line offsets so that we can jump to a position in file. */
	static EXRREADERGPU_API bool ReadLineOrTileOffsets(FILE* FileHandle, ELineOrder LineOrder, TArray<int64>& LineOrTileOffsets);

public:

	/** Discards the header and fills the buffer with plain pixel data. */
	static EXRREADERGPU_API bool GenerateTextureData(uint16* Buffer, int32 BufferSize, FString FilePath, int32 NumberOfScanlines, int32 NumChannels);

	/**
	 * To be able to read tiles, especially in cases where tiles vary in sizes, we need to calculate offsets manually
	 * for custom exr files, since we don't write out offsets.
	*/
	static EXRREADERGPU_API void CalculateTileOffsets
		( TArray<int32>& OutNumTilesPerLevel
		, TArray<TArray<FTileDesc>>& OutPartialTileInfo
		, const FIntPoint& FullTextureResolution
		, const FIntPoint& TileDimWithBorders
		, int32 NumMipLevels
		, int64 PixelSize);

	/** 
	* This function is used to open file and keep a handle to it so the file can be read in chunks.
	* Reading in chunks allows the process of reading to be canceled midway through reading. 
	* This function reads and discards the header and keeps the pointers to the scanlines or tiles. 
	*/
	EXRREADERGPU_API bool OpenExrAndPrepareForPixelReading(FString FilePath, const TArray<int32>& NumOffsetsPerLevel);

	/** 
	* Read a chunk of an Exr file previously open via OpenExrAndPrepareForPixelReading.
	* The read starts from the last point of file reading.
	*/
	EXRREADERGPU_API bool ReadExrImageChunk(void* Buffer, int64 ChunkSize);

	/**
	* Moves to the specified tile position in the file in preparation for reading.
	*/
	EXRREADERGPU_API bool SeekTileWithinFile(const int32 StartTileIndex, const int32 Level, int64& OutBufferOffset);

	/**
	* Gets byte location of the requested tile for the requested mip level.
	*/
	EXRREADERGPU_API bool GetByteOffsetForTile(const int32 TileIndex, const int32 Level, int64& OutBufferOffset);

	/**
	* After we have finished reading the file that was open via OpenExrAndPrepareForPixelReading
	* We need to close it to avoid memory leaks
	*/
	EXRREADERGPU_API bool CloseExrFile();

private:

	/**
	* One approach to reading is to first open a file, read in chunks and then close the file.
	* It is all done in 3 sepate functions and requires to keep track of Filehandle
	*/
	FILE* FileHandle;

	/**
	* The length of the file in question. Used to get the last tile.
	*/
	int64 FileLength;

	/**
	* These are byte offsets pointing to scanlines or tiles from the begining of the file.
	*/
	TArray<TArray<int64>> LineOrTileOffsetsPerLevel;
};
#endif
