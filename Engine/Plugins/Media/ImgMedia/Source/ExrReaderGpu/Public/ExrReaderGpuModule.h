// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogExrReaderGpu, Log, All);

class FExrReaderGpuModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation start */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	/** IModuleInterface implementation end */
};

