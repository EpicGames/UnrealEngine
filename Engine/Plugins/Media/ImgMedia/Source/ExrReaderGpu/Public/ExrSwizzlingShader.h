// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if defined(PLATFORM_WINDOWS) && PLATFORM_WINDOWS

#include "CoreMinimal.h"
#include "GlobalShader.h"
#include "ShaderParameterUtils.h"
#include "ShaderParameterStruct.h"

// The vertex shader used by DrawScreenPass to draw a rectangle.
class FExrSwizzleVS : public FGlobalShader
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FExrSwizzleVS, EXRREADERGPU_API);

	FExrSwizzleVS() = default;
	FExrSwizzleVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};

/** Pixel shader swizzle RGB planar buffer data into proper RGBA texture. */
class FExrSwizzlePS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FExrSwizzlePS, EXRREADERGPU_API);
	SHADER_USE_PARAMETER_STRUCT(FExrSwizzlePS, FGlobalShader);

	/** If the provided buffer is RGBA the shader would work slightly differently to RGB. */
	class FRgbaSwizzle : SHADER_PERMUTATION_INT("PERMUTATION_CHANNELS", 4);
	class FRenderTiles : SHADER_PERMUTATION_BOOL("RENDER_TILES");
	class FPartialTiles : SHADER_PERMUTATION_BOOL("PARTIAL_TILES");
	using FPermutationDomain = TShaderPermutationDomain<FRgbaSwizzle, FRenderTiles, FPartialTiles>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_SRV(StructuredBuffer<uint>, UnswizzledBuffer)
		SHADER_PARAMETER_SRV(StructuredBuffer<FTileDesc>, TileDescBuffer)
		SHADER_PARAMETER(FIntPoint, TextureSize)
		SHADER_PARAMETER(FIntPoint, TileSize)
		SHADER_PARAMETER(FIntPoint, NumTiles)
		SHADER_PARAMETER(int32, NumChannels)
		SHADER_PARAMETER(uint32, bApplyColorTransform)
		SHADER_PARAMETER(uint32, EOTF)
		SHADER_PARAMETER(FMatrix44f, ColorSpaceMatrix)
	END_SHADER_PARAMETER_STRUCT()
};
#endif