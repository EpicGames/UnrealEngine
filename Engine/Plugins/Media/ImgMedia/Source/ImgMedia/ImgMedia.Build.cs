// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ImgMedia : ModuleRules
	{
		public ImgMedia(ReadOnlyTargetRules Target) : base(Target)
		{
			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"ImageCore",
					"ImgMediaEngine",
					"ImgMediaFactory",
					"MediaUtils",
					"MovieScene",
					"RenderCore",
					"Renderer",
					"RHI",
					"Slate",
					"SlateCore",
					"TmvMedia"
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"MediaAssets",
					"TimeManagement",
				});

			bool bLinuxEnabled = Target.Platform == UnrealTargetPlatform.Linux && Target.Architecture == UnrealArch.X64;

			if ((Target.Platform == UnrealTargetPlatform.Mac) ||
				Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) ||
				bLinuxEnabled)
			{
				PrivateDependencyModuleNames.Add("OpenExrWrapper");
				PrivateDependencyModuleNames.Add("ExrReaderGpu");
			}
			
			// Is this the editor?
			if (Target.bBuildEditor == true)
			{
				PublicDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}
