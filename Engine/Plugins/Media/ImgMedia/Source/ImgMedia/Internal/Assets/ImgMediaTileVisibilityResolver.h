// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Assets/MediaSequenceInfo.h"
#include "Assets/MediaTileVisibility.h"
#include "Containers/Array.h"
#include "HAL/CriticalSection.h"
#include "Templates/SharedPointer.h"

/**
 * Per-loader manager that gathers tile-visibility requests from registered
 * providers and produces a merged snapshot for loader workers to consume.
 */
class FImgMediaTileVisibilityResolver
{
public:
	FImgMediaTileVisibilityResolver();

	/** Unsubscribes from every live provider so a late broadcast cannot reach the dying resolver. */
	~FImgMediaTileVisibilityResolver();

	/** Set the sequence (size, tiling, mip count) used when resolving tiles. Invalidates the snapshot. */
	void SetSequenceInfo(const FMediaSequenceInfo& InInfo);

	/** Get the current sequence info. Thread-safe. */
	FMediaSequenceInfo GetSequenceInfo() const;

	/** Add a provider. Deduplicates against existing weak refs; invalidates the snapshot on success. */
	void RegisterProvider(TSharedRef<IMediaTileVisibilityProvider> InProvider);

	/** Remove a provider. Safe to call when not registered; invalidates the snapshot. */
	void UnregisterProvider(const TSharedRef<IMediaTileVisibilityProvider>& InProvider);

	/** Drop all registered providers and the cached snapshot. */
	void ClearProviders();

	/** Count of currently-alive providers (expired weak refs are pruned by this call). */
	int32 NumProviders() const;

	/** True if at least one alive provider is registered. */
	bool HasProviders() const;

	/** Force the next GetSnapshot to rebuild. Called once per frame from FImgMediaLoader::TickFrame. */
	void InvalidateSnapshot();

	/** Get the merged tile-visibility request, lazily rebuilt if invalidated. Thread-safe. */
	TSharedPtr<FMediaTileVisibilityRequest, ESPMode::ThreadSafe> GetSnapshot();

	/** Convenience: returns the snapshot's MipLevelToUpscale, or -1 if disabled. */
	int32 GetMinimumMipLevelToUpscale();

	/** Push the current snapshot's mip list and tile count to the on-screen debug overlay. */
	void AddVisibilitySnapshotDebugOverlay();

private:
	struct FProviderEntry
	{
		TWeakPtr<IMediaTileVisibilityProvider> Weak;
		FDelegateHandle OnInputsChangedHandle;
	};

	/** Removes the OnInputsChanged subscription if the provider is still alive and clears the handle. */
	static void UnsubscribeFromProvider(FProviderEntry& InOutEntry);

	/** Guards Providers, SequenceInfo, and CachedSnapshot. */
	mutable FCriticalSection Mutex;

	/** Registered providers held weakly; the consumer owns the strong reference. */
	TArray<FProviderEntry> Providers;

	/** The sequence we resolve tiles for. */
	FMediaSequenceInfo SequenceInfo;

	/** Lazily-built merged snapshot returned by GetSnapshot; null when invalidated. */
	TSharedPtr<FMediaTileVisibilityRequest, ESPMode::ThreadSafe> CachedSnapshot;
};
