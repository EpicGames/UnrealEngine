// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/LruCache.h"
#include "Containers/Queue.h"
#include "IMediaEventSink.h"
#include "IMediaSamples.h"
#include "IMediaTextureSample.h"
#include "IMediaView.h"
#include "Assets/ImgMediaTileVisibilityResolver.h"
#include "Misc/FrameRate.h"
#include "Templates/SharedPointer.h"

#include <atomic>

class FImgMediaGlobalCache;
class FImgMediaLoaderWork;
class FImgMediaScheduler;
class FImgMediaTextureSample;
class FMediaTimeStamp;
class IImgMediaReader;
class IMediaEventSink;
class IQueuedWork;

struct FImgMediaFrame;
struct FImgMediaFrameInfo;
struct FNativeMediaSourceColorSettings;

/**
 * Settings for the smart cache.
 */
struct FImgMediaLoaderSmartCacheSettings
{
	/** True if we are using the smart cache. */
	bool bIsEnabled;
	/** The cache will fill up with frames that are up to this time from the current time. */
	float TimeToLookAhead;

	/** Constructor. */
	FImgMediaLoaderSmartCacheSettings(bool bInIsEnabled, float InTimeToLookAhead)
		: bIsEnabled(bInIsEnabled)
		, TimeToLookAhead(InTimeToLookAhead)
	{
	}
};

/**
 * Sequence playback bandwidth requirements and estimates.
 */
struct FImgMediaLoaderBandwidth
{
	/** Number of bytes read per second from the last frame read. */
	float Current;
	/** Average bandwidth over time (including idle time). */
	float Effective;
	/** Minimum number of bytes per second needed to read all mip levels. */
	float Required;

	/** Constructor. */
	FImgMediaLoaderBandwidth();

	/** Updated calculated current and effective bandwidth numbers. */
	void Update(const TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>& Frame, double WorkTime);

	/** Empty the read times cache and resets its index. */
	void EmptyCache();

private:
	/** Work time cache for average bandwidth calculation. */
	TArray<TPair<double, double>> ReadTimeCache;
	/** Circular index for work time cache calculation. */
	int32 ReadTimeCacheIndex;
};

/**
 * Loads image sequence frames from disk.
 */
class FImgMediaLoader
	: public TSharedFromThis<FImgMediaLoader, ESPMode::ThreadSafe>
{
public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InScheduler               The scheduler for image loading.
	 * @param InGlobalCache             The global frame cache shared across loaders.
	 * @param InSourceColorSettings     Color space and encoding overrides for the source, or null for defaults.
	 * @param bInFillGapsInSequence     If true, gaps in the image sequence are filled with blank frames.
	 * @param InLayerName               EXR layer to select. Empty string reads the default (un-prefixed) channels.
	 * @param InSmartCacheSettings      Settings controlling the smart cache pre-fetching behavior.
	 * @param InEventSink               Optional event sink for Internal_RequestVideoSampleRefresh; the
	 *                                  caller owns the sink and must keep it alive. Null disables.
	 * @param InPlayerGuid              Player's identifier (same value lives on FMediaPlayerFacade and FImgMediaPlayer);
	 *                                  used in log lines for cross-layer correlation. Defaults to an empty GUID.
	 */
	FImgMediaLoader(const TSharedRef<FImgMediaScheduler, ESPMode::ThreadSafe>& InScheduler,
		const TSharedRef<FImgMediaGlobalCache, ESPMode::ThreadSafe>& InGlobalCache,
		const TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe>& InSourceColorSettings,
		bool bInFillGapsInSequence,
		const FString& InLayerName,
		const FImgMediaLoaderSmartCacheSettings& InSmartCacheSettings,
		IMediaEventSink* InEventSink = nullptr,
		const FGuid& InPlayerGuid = FGuid());

	/** Virtual destructor. */
	virtual ~FImgMediaLoader();

	/**
	 * Per-frame tick driven from FImgMediaPlayer::TickInput. Invalidates the
	 * tile-visibility snapshot so the next frame's first worker rebuilds it,
	 * and (when enabled) draws the debug overlay for the previous frame's
	 * snapshot. Called on the game thread, bounded by the player's lifetime.
	 */
	void TickFrame();

public:
	/** Max number of mip map levels supported. */
	static constexpr int32 MAX_MIPMAP_LEVELS = 32;

	/**
	 * Get the data bit rate of the video frames.
	 *
	 * @return Data rate (in bits per second).
	 */
	uint64 GetBitRate() const;

#if WITH_EDITOR
	/** Get the number of bytes read per second from the last frame read. */
	float GetCurrentBandwidth() const { return Bandwidth.Current; }
	/** Get the effective bandwidth. */
	float GetEffectiveBandwidth() const { return Bandwidth.Effective; }
	/** Get the minimum number of bytes per second needed to read all mip levels. */
	float GetRequiredBandwidth() const { return Bandwidth.Required; }
#endif

	/**
	 * Get the time ranges of frames that are being loaded right now.
	 *
	 * @param OutRangeSet Will contain the set of time ranges.
	 * @return true on success, false otherwise.
	 * @see GetCompletedTimeRanges, GetPendingTimeRanges
	 */
	void GetBusyTimeRanges(TRangeSet<FTimespan>& OutRangeSet) const;

	/**
	 * Get the time ranges of frames that are are already loaded.
	 *
	 * @param OutRangeSet Will contain the set of time ranges.
	 * @return true on success, false otherwise.
	 * @see GetBusyTimeRanges, GetPendingTimeRanges
	 */
	void GetCompletedTimeRanges(TRangeSet<FTimespan>& OutRangeSet) const;

	/**
	 * Get the information string for the currently loaded image sequence.
	 *
	 * @return Info string.
	 * @see GetSequenceDuration, GetSamples
	 */
	const FString& GetInfo() const
	{
		return Info;
	}

	/**
	 * Notify loader out a seek
	 */
	void Seek(const FMediaTimeStamp& SeekTarget, bool bReverse);

	/**
	 * Tries to get the best sample for a given time range.
	 *
	 * @param TimeRange is the range to check for.
	 * @param OutSample will be filled in with the sample if found.
	 * @param bIsLoopingEnabled True if we can loop.
	 * @param PlayRate How fast we are playing.
	 * @return Return code indicating status (ok, unsupported, no sample)
	 */
	IMediaSamples::EFetchBestSampleResult FetchBestVideoSampleForTimeRange(const TRange<FMediaTimeStamp>& TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample, bool bIsLoopingEnabled, float PlayRate);

	/**
	 * Checks to see if a sample is available at the specificed time.
	 *
	 * @param TimeStamp Will be filled in with the time stamp of the sample if a sample is found.
	 * @param bIsLoopingEnabled True if we can loop.
	 * @param PlayRate How fast we are playing.
	 * @param CurrentTime Time to get a sample for.
	 * @return True if a sample is available.
	 */
	bool PeekVideoSampleTime(FMediaTimeStamp& TimeStamp, bool bIsLoopingEnabled, float PlayRate, const FTimespan& CurrentTime);

	/**
	 * Discard any samples in the given range from the output "queue" (logically)
	 */
	bool DiscardVideoSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bIsLoopingEnabled, float PlayRate);

	/**
	 * Get the time ranges of frames that are pending to be loaded.
	 *
	 * @param OutRangeSet Will contain the set of time ranges.
	 * @return true on success, false otherwise.
	 * @see GetBusyTimeRanges, GetCompletedTimeRanges
	 */
	void GetPendingTimeRanges(TRangeSet<FTimespan>& OutRangeSet) const;

	/**
	* Returns true if the reader is valid.
	* Used to validate if it is safe to call GetReader().
	*/
	bool IsReaderValid() const
	{
		return Reader.IsValid();
	}

	/**
	 * Get the image reader object used by this loader.
	 *
	 * @return The image reader.
	 */
	TSharedRef<IImgMediaReader, ESPMode::ThreadSafe> GetReader() const
	{
		return Reader.ToSharedRef();
	}

	/**
	 * Get the tile-visibility resolver used by this loader.
	 * Always non-null; consumers register providers on this to influence
	 * which tiles get loaded. When no providers register, the loader falls
	 * back to loading every tile at every mip level.
	 */
	const TSharedPtr<FImgMediaTileVisibilityResolver, ESPMode::ThreadSafe>& GetTileVisibilityResolver() const
	{
		return TileVisibilityResolver;
	}

	/** Player identifier mirrored from FImgMediaPlayer; useful for cross-layer log correlation. */
	const FGuid& GetPlayerGuid() const { return PlayerGuid; }

	/**
	 * Get the width and height of the image sequence.
	 *
	 * The dimensions of the image sequence are determined by reading the attributes
	 * of the first image. The dimensions of individual image frames in the sequence
	 * are allowed to differ. However, this usually indicates a mistake in the content
	 * creation pipeline and will be logged out as such.
	 *
	 * @return Sequence dimensions.
	 * @see GetSequenceDuration, GetSequenceFps
	 */
	const FIntPoint& GetSequenceDim() const
	{
		return SequenceDim;
	}

	/**
	 * Get the total duration of the image sequence.
	 *
	 * @return Duration.
	 * @see GetSequenceDim, GetSequenceFps
	 */
	FTimespan GetSequenceDuration() const
	{
		return SequenceDuration;
	}

	/**
	 * Get the sequence's frame rate.
	 *
	 * The frame rate of the image sequence is determined by reading the attributes
	 * of the first image. Individual image frames may specify a different frame rate,
	 * but it will be ignored during playback.
	 *
	 * @return The frame rate.
	 * @see GetSequenceDuration, GetSequenceDim
	 */
	FFrameRate GetSequenceFrameRate() const
	{
		return SequenceFrameRate;
	}

	/**
	 * Get the path to images in the sequence.
	 *
	 * @param FrameNumber Frame to get.
	 * @param MipLevel Mip level to get.
	 * @return Returns path and filename of image.
	 */
	const FString& GetImagePath(int32 FrameNumber, int32 MipLevel) const;

	/** Gets the layer name being read in the sequence, if the images are EXR images */
	const FString& GetLayerName() const { return LayerName; }

	/**
	 * Get the number of mipmap levels we have.
	 */
	int32 GetNumMipLevels() const 
	{ 
		return NumMipLevels; 
	};

	/**
	 * Get the number of bytes per pixel for each image.
	 */
	SIZE_T GetNumBytesPerPixel() const
	{
		return NumBytesPerPixel;
	}

	/**
	 * Get the minumum mip level to upscale to lower quality mips (-1 if disabled).
	 */
	int32 GetMinimumLevelToUpscale() const
	{
		return MinimumMipLevelToUpscale;
	};

	/**
	 * If mips are stored in separate files we should make sure to read it.
	 */
	bool MipsInSeparateFiles() const
	{
		return ImagePaths.Num() > 1;
	};

	/**
	 * Get the number of images/frames in a single mip level.
	 *
	 * @return Number of images.
	 */
	int32 GetNumImages() const
	{
		if (bIsContainerFile)
		{
			return NumContainerFrames;
		}
		return ImagePaths.Num() > 0 ? ImagePaths[0].Num() : 0;
	}

	/** Whether the source is a container file (e.g. .tmv) rather than a file sequence. */
	bool IsContainerFile() const
	{
		return bIsContainerFile;
	}

	/**
	 * Get the number of tiles in the X direction.
	 */
	int32 GetNumTilesX() const
	{
		return TilingDescription.TileNum.X;
	}

	/**
	 * Get the number of tiles in the Y direction.
	 */
	int32 GetNumTilesY() const
	{
		return TilingDescription.TileNum.Y;
	}

	bool IsTiled() const
	{
		return (TilingDescription.TileNum.X > 1) || (TilingDescription.TileNum.Y > 1);
	}

	/**
	 * Get the next work item.
	 *
	 * This method is called by the scheduler.
	 *
	 * @return The work item, or nullptr if no work is available.
	 */
	IQueuedWork* GetWork();

	/**
	 * Initialize the image sequence loader.
	 *
	 * @param SequencePath Path to the image sequence.
	 * @param FrameRateOverride The frame rate to use (0/0 = do not override).
	 * @param Loop Whether the cache should loop around.
	 * @param bIsSequencerEnvironment Whether image sequence playback comes from sequencer, to decide on warmup behavior.
	 * @see IsInitialized
	 */
	void Initialize(const FString& SequencePath, const FFrameRate& FrameRateOverride, bool Loop, bool bIsSequencerEnvironment);

	/**
	 * Whether this loader has been initialized yet.
	 *
	 * @return true if initialized, false otherwise.
	 * @see Initialize
	 */
	bool IsInitialized() const
	{
		return Initialized;
	}

	/**
	 * Notify the loader that a work item completed.
	 *
	 * @param CompletedWork The work item that completed.
	 * @param FrameNumber Number of the frame that was read.
	 * @param Frame The frame that was read, or nullptr if reading failed.
	 * @param WorkTime How long to read this frame (in seconds).
	 */
	void NotifyWorkComplete(FImgMediaLoaderWork& CompletedWork, int32 JobID, int32 FrameNumber,
		const TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>& Frame, float WorkTime);

	/**
	 * Asynchronously request the image frame at the specified time.
	 *
	 * @param Time The time of the image frame to request (relative to the beginning of the sequence).
	 * @param PlayRate The current play rate (used by the look-ahead logic).
	 * @param Loop Whether the cache should loop around.
	 * @return bool if the frame will be loaded, false otherwise.
	 */
	bool RequestFrame(FTimespan Time, float PlayRate, bool Loop);

	/**
	 * Flush state if queue emulation
	 */
	void Flush();

	/**
	 * Tell the loader if playback is blocking.
	 */
	void SetIsPlaybackBlocking(bool bIsBlocking);

	/** Mirror the owning player's paused state into the loader; used by Update / IsCachedFrameInRange. */
	void SetIsPaused(bool bInIsPaused);

	/** Query if time represents last first frame or before */
	bool IsFrameFirst(const FTimespan& TimeStamp) const;

	/** Query if time represents last last frame or later */
	bool IsFrameLast(const FTimespan& TimeStamp) const;

	/**
	 * Source color settings getter.
	 */
	TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> GetSourceColorSettings() const { return SourceColorSettings; }

protected:
	struct FFrameNumberJobInfo
	{
		FFrameNumberJobInfo() = default;
		FFrameNumberJobInfo(int32 InFrame, int32 InJobID) : Frame(InFrame), JobID(InJobID) {}

		int32 Frame;
		int32 JobID;
		bool operator==(const FFrameNumberJobInfo& Other) const { return JobID == Other.JobID && Frame == Other.Frame; }
	};

	/**
	 * Convert a collection of frame numbers to corresponding time ranges.
	 *
	 * @param FrameNumbers The frame numbers to convert.
	 * @param OutRangeSet Will contain the time ranges.
	 */
	void FrameNumbersToTimeRanges(const TArray<int32>& FrameNumbers, TRangeSet<FTimespan>& OutRangeSet) const;

	/**
	 * Get the play head time corresponding to the specified frame number.
	 *
	 * @param FrameNumber The frame number.
	 * @return The corresponding time.
	 * @see TimeToFrameNumber
	 */
	FTimespan FrameNumberToTime(uint32 FrameNumber) const;

	/**
	 * Initialize the image sequence loader.
	 *
	 * @param SequencePath Path to the image sequence.
	 * @param FrameRateOverride The frame rate to use (0/0 = do not override).
	 * @param Loop Whether the cache should loop around.
	 * @param OutFirstFrameInfo Sequence information based on its first frame.
	 * @return True when loading succeeds.
	 */
	bool LoadSequence(const FString& SequencePath, const FFrameRate& FrameRateOverride, bool Loop, FImgMediaFrameInfo& OutFirstFrameInfo);

	/**
	 * Warms up the player and start issuing requests from the start of the sequence.
	 *
	 * @param InFirstFrameInfo Sequence information based on its first frame.
	 * @param Loop Whether the cache should loop around.
	 * @param bIsSequencerEnvironment Whether image sequence playback comes from sequencer.
	 */
	void WarmupSequence(const FImgMediaFrameInfo& InFirstFrameInfo, bool Loop, bool bIsSequencerEnvironment);

	/**
	 * Finds all the files in a directory and gets their path.
	 *
	 * @param SequencePath Directory to look in.
	 * @param OutputPaths File paths will be added to this.
	 */
	void FindFiles(const FString& SequencePath, TArray<FString>& OutputPaths);

	/**
	 * Finds the mip map files for this sequence (if any).
	 *
	 * Typically with non mips, a single directory holds all the files of a single sequence.
	 *
	 * With mip maps, a directory will hold all the files of a single sequence of a specific mip level.
	 * The naming convention is for the directory name to end in _<SIZE>.
	 *   SIZE does not need to be a power of 2.
	 *   Each subsequent level should have SIZE be half of the level preceeding it.
	 *   If SIZE does not divide evenly by 2, then round down.
	 * The part of the name preceding _<SIZE> should be the same for all mip levels.
	 * All mip levels of the same sequence should be in the same location
	 * E.g. /Sequence/Seq_256/, /Sequence/Seq_128/, /Sequence/Seq_64/, etc.

	 * FindMips will look for mip levels that are the at the level of SequencePath and below.
	 * E.g. if SequencePath is Seq_1024, then FindMips will look for Seq_1024, Seq_512, etc
	 * and will NOT look for Seq_2048 even if it is present.
	 *
	 * @param SequencePath Directory of sequence.
	 */
	void FindMips(const FString& SequencePath);

	/**
	 * Get the frame number corresponding to the specified play head time.
	 *
	 * @param Time The play head time.
	 * @return The corresponding frame number, or INDEX_NONE.
	 * @see FrameNumberToTime
	 */
	uint32 TimeToFrameNumber(FTimespan Time) const;

	/**
	 * Get the frame number corresponding to the specified play head time without any checks for media range bounds.
	 *
	 * @param Time The play head time.
	 * @return The corresponding frame number, or INDEX_NONE.
	 * @see FrameNumberToTime
	 */
	int64 TimeToFrameNumberUnbound(FTimespan Time) const;

	/**
	 * Update the loader based on the current play position.
	 *
	 * @param PlayHeadFrame Current play head frame number.
	 * @param PlayRate Current play rate.
	 * @param Loop Whether the cache should loop around.
	 */
	void Update(int32 PlayHeadFrame, float PlayRate, bool Loop);

	/**
	 * Adds an empty frame to the cache if no frame is present at the specified number.
	 * 
	 * @param FrameNumber Frame number to add.
	 * @return True if the frame was added
	 */
	bool TryAddEmptyFrame(int32 FrameNumber);

	/**
	 * Adds a frame to the cache.
	 *
	 * @param FrameNumber Number of frame to add.
	 * @param Frame Frame information to add.
	 */
	void AddFrameToCache(int32 FrameNumber, const TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>& Frame);

	/**
	 * Get what mip level and tiles we should be using for a given frame.
	 *
	 * @param FrameIndex Frame to get mip level for.
	 * @param OutMipsAndTiles Will be filled in with what tiles are actually needed.
	 */
	void GetDesiredMipTiles(int32 FrameIndex, TMap<int32, FMediaTileSelection>& OutMipsAndTiles) const;

	/**
	 * Get what mip level supporting readers should upscale.
	 *
	 * @return minimum mip level to upscale.
	 */
	int32 GetDesiredMinimumMipLevelToUpscale() const;

	/***
	 * Modulos the time so that it is between 0 and SequenceDuration.
	 * Handles negative numbers appropriately.
	 */
	FTimespan ModuloTime(FTimespan Time) const;

	/**
	 * Helper function to get the number at the end of a string.
	 * 
	 * The number does not have to be at the very end,
	 * e.g. test1.exr will get 1.
	 * 
	 * @param Number Will be set to the number found if one is found.
	 * @param String String to search.
	 * @param Extension If the filename has an extension, it will be ignored (in case there is a number in the extension).
	 * @return True if it found a number.
	 */
	bool GetNumberAtEndOfString(int32& Number, const FString& String, const FString& Extension) const;

	/**
	 * Convert FrameIndex to a frame that fits with our skipping logic.
	 * E.g. if SkipFramesLevel is 2, then this will return frames 4 frames apart.
	 * E.g. FrameIndex	Return value
	 *		0			3
	 *		1			3
	 *		2			3
	 *		3			3
	 *		4			7
	 *		5			7
	 */
	int32 GetSkipFrame(int32 FrameIndex) const;

	/**
	 * Updates how we throttle bandwidth depending on current settings.
	 */
	void UpdateBandwidthThrottling();

	/**
	 * Get frame from either the local or global cache, nullptr if not found.
	 */
	const TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>* GetCachedFrame(int32 InFrameNumber);

	/**
	 * Reset "queued fetch" related state used to emulate player output queue behavior
	 */
	void ResetFetchLogic();

	/**
	 * Check whether the current sequence has valid mip levels or tiles, and a valid mip map info object.
	 */
	bool HasMipsOrTilesEnabled() const;

private:

	/** Critical section for synchronizing access to Frames. */
	mutable FCriticalSection CriticalSection;

	/** The currently loaded image sequence frames. */
	TLruCache<int32, TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>> Frames;

	/**
	 * Paths to each image for each mip map level in the currently opened sequence.
	 * This is an array of mip levels, and each mip level is an array of image paths.
	 * Not used for container files.
	 */
	TArray<TArray<FString>> ImagePaths;

	/** Whether the source is a container file rather than a file sequence. */
	bool bIsContainerFile = false;

	/** Number of frames in the container (only valid when bIsContainerFile is true). */
	int32 NumContainerFrames = 0;

	/** Media information string. */
	FString Info;

	/** Whether this loader has been initialized yet. */
	bool Initialized;

	/** If true, then any gaps in the sequence will be filled with blank frames. */
	const bool bFillGapsInSequence;

	/** Tiling description. */
	FMediaTextureTilingDescription TilingDescription;

	/** The number of frames to load ahead of the play head. */
	int32 NumLoadAhead;

	/** The number of frames to load behind the play head. */
	int32 NumLoadBehind;

	/** Total number of frames to load. */
	int32 NumFramesToLoad;

	/** Stores number of mip levels. */
	int32 NumMipLevels;

	/** Stores number of bytes per pixel for each image. */
	SIZE_T NumBytesPerPixel;

	/** Minumum mip level to upscale. */
	TAtomic<int32> MinimumMipLevelToUpscale;

	/** The image sequence reader to use. */
	TSharedPtr<IImgMediaReader, ESPMode::ThreadSafe> Reader;

	/** The scheduler for image loading. */
	TSharedPtr<FImgMediaScheduler, ESPMode::ThreadSafe> Scheduler;

	/** The scheduler for image loading. */
	TSharedPtr<FImgMediaGlobalCache, ESPMode::ThreadSafe> GlobalCache;

	/** Tile-visibility resolver. Always created; dormant until consumers register providers. */
	TSharedPtr<FImgMediaTileVisibilityResolver, ESPMode::ThreadSafe> TileVisibilityResolver;

	/** Container for media source color settings. */
	TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> SourceColorSettings;

	/** Name of the EXR layer to read. */
	FString LayerName;

	/** Width and height of the image sequence (in pixels) .*/
	FIntPoint SequenceDim;

	/** Total length of the image sequence. */
	FTimespan SequenceDuration;

	/** Frame rate of the currently loaded sequence. */
	FFrameRate SequenceFrameRate;

	/** Identifying name of sequence files. */
	FName SequenceName;

private:
	bool RequestFrame(int32 FrameNumber, float PlayRate, bool Loop);

	/** When using a global cache, handles the case when the last requested frame might have been evicted. */
	void HandleGlobalCacheFrameEviction();
	
	/** Index of the previously requested frame. */
	int32 LastRequestedFrame;

	/** Loop flag stashed from the last RequestFrame so TickFrame's paused-replay Update sees the right wrap-around state. */
	bool LastRequestedLoop = false;

	/** Player-supplied sink for Internal_RequestVideoSampleRefresh events. Outlives the loader; null disables. */
	IMediaEventSink* EventSink = nullptr;

	/** Player identifier (mirrors FImgMediaPlayer::PlayerGuid); included in log lines for cross-layer correlation. */
	FGuid PlayerGuid;

	/** Mirror of the owning player's paused state; pushed via SetIsPaused. Atomic because NotifyWorkComplete reads it from worker threads. */
	std::atomic<bool> bIsPaused{false};

	/** Counter used to retry loading frames again. */
	int32 RetryCount;

	/** Collection of frame numbers that need to be read. */
	TArray<int32> PendingFrameNumbers;

	/** Collection of frame numbers that are being read. */
	TArray<FFrameNumberJobInfo> QueuedFrameNumbers;
	int32 NextFrameNumberJobID;

	/** Object pool for reusable work items. */
	TArray<FImgMediaLoaderWork*> WorkPool;

	/** True if we are using the global cache, false to use the local cache. */
	bool UseGlobalCache;

	/** State related to "queue style" frame access functions */
	struct FQueuedSampleFetch
	{
		FMediaTimeStamp LastTimeStamp;	// Current playhead / last delivered sample PTS
		FTimespan LastDuration;			// Duration of a frame / zero after seek or playback start
		int32 LoopIndex;				// Current loop index to be encoded into sequence index of timestamp
	};
	FQueuedSampleFetch QueuedSampleFetch;

private:

	/** Settings for the smart cache. */
	FImgMediaLoaderSmartCacheSettings SmartCacheSettings;
	/** True if the player is using blocked playback. */
	bool bIsPlaybackBlocking;

	/** If we want to throttle bandwidth when there is not enough bandwidth. */
	bool bIsBandwidthThrottlingEnabled;
	/** If we can skip frames to throttle bandwidth. */
	bool bIsSkipFramesEnabled;
	/** Keeps a count so we can raise/lower the skip level when this gets too high/low. */
	int32 SkipFramesCounter;
	/**
	 * The current level of skipping.
	 * The frame increment to the next frame is 1 << SkipFramesLevel.
	 * So level 0 -> next frame is +1, i.e. the next frame, i.e. no skipping.
	 *    level 1 -> next frame is +2, i.e. we skip 1 frame.
	 *    level 2 -> next frame is +4, i.e. we skip 3 frames.
	 */
	int32 SkipFramesLevel;
	/** When SkipFramesCounter is greater than this, then increase SkipFramesLevel. */
	const int32 SkipFramesCounterRaiseLevelThreshold = 10;
	/** When SkipFramesLevel changes, the value of SkipFramesCounter will reset to this. */
	const int32 SkipFramesCounterNewLevelValue = 5;

#if WITH_EDITOR
	/** Bandwidth estimation. */
	FImgMediaLoaderBandwidth Bandwidth;
#endif
};
