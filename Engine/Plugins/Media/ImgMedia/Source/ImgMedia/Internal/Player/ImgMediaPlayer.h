// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMediaCache.h"
#include "IMediaControls.h"
#include "IMediaPlayer.h"
#include "IMediaSamples.h"
#include "IMediaTracks.h"
#include "IMediaView.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FImgMediaLoader;
class IMediaTileVisibilityProvider;
class FImgMediaScheduler;
class IImgMediaReader;
class IMediaEventSink;
class IMediaTextureSample;
class FImgMediaGlobalCache;
class UMediaTexture;

#define UE_API IMGMEDIA_API

/**
 * Implements a media player for image sequences.
 */
class FImgMediaPlayer final
	: public IMediaPlayer
	, protected IMediaCache
	, protected IMediaControls
	, protected IMediaSamples
	, protected IMediaTracks
	, public IMediaView
{
public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InEventSink The object that receives media events from this player.
	 * @param InScheduler The image loading scheduler to use.
	 * @param InGlobalCache Optional global cache to use for this player.
	 */
	FImgMediaPlayer(IMediaEventSink& InEventSink, const TSharedRef<FImgMediaScheduler, ESPMode::ThreadSafe>& InScheduler,
		const TSharedRef<FImgMediaGlobalCache, ESPMode::ThreadSafe>& InGlobalCache);

	/** Virtual destructor. */
	virtual ~FImgMediaPlayer() override;

	/** Get the loader. */
	TSharedPtr<FImgMediaLoader, ESPMode::ThreadSafe> GetLoader() { return Loader; }

	//~ Begin IMediaPlayer
	virtual void Close() override;
	virtual IMediaCache& GetCache() override;
	virtual IMediaControls& GetControls() override;
	virtual FString GetInfo() const override;
	virtual FVariant GetMediaInfo(FName InfoName) const override;
	virtual FGuid GetPlayerPluginGUID() const override;
	virtual IMediaSamples& GetSamples() override;
	virtual FString GetStats() const override;
	virtual IMediaTracks& GetTracks() override;
	UE_API virtual FString GetUrl() const override;
	virtual IMediaView& GetView() override;
	virtual bool Open(const FString& Url, const IMediaOptions* Options) override;
	virtual bool Open(const FString& Url, const IMediaOptions* Options, const FMediaPlayerOptions* PlayerOptions) override;
	virtual bool Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* Options) override;
	virtual void SetGuid(const FGuid& Guid) override { PlayerGuid = Guid; }
	virtual void TickInput(FTimespan DeltaTime, FTimespan Timecode) override;
	virtual bool FlushOnSeekStarted() const override;
	virtual bool FlushOnSeekCompleted() const override;
	virtual bool GetPlayerFeatureFlag(EFeatureFlag flag) const override;
	
#if WITH_EDITOR
	virtual void RecordSourceOpenedEvent() const override;
	virtual bool ShouldAutoRecordSourceOpenedEvent() const override { return false; }
#endif
	//~ End IMediaPlayer

	/** Facade-supplied identifier; same value lives on FMediaPlayerFacade::PlayerGuid for cross-layer log correlation. */
	const FGuid& GetPlayerGuid() const { return PlayerGuid; }

protected:
	/**
	 * Check whether this player is initialized.
	 *
	 * @return true if initialized, false otherwise.
	 */
	bool IsInitialized() const;

public:
	//~ Begin IMediaCache
	virtual bool QueryCacheState(EMediaCacheState State, TRangeSet<FTimespan>& OutTimeRanges) const override;
	//~ End IMediaCache

	//~ Begin IMediaControls
	virtual bool CanControl(EMediaControl Control) const override;
	virtual FTimespan GetDuration() const override;
	virtual float GetRate() const override;
	virtual EMediaState GetState() const override;
	virtual EMediaStatus GetStatus() const override;
	virtual TRangeSet<float> GetSupportedRates(EMediaRateThinning Thinning) const override;
	virtual FTimespan GetTime() const override;
	virtual bool IsLooping() const override;
	virtual bool Seek(const FTimespan& Time) override
	{ check(!"You have to call the override with additional options!"); return false; }
	virtual bool Seek(const FTimespan& InNewTime, const FMediaSeekParams& InAdditionalParams) override;
	virtual bool SetLooping(bool Looping) override;
	virtual bool SetRate(float Rate) override;
	virtual void SetBlockingPlaybackHint(bool bFacadeWillUseBlockingPlayback) override;
	//~ End IMediaControls

	//~ Begin IMediaSamples
	virtual bool FetchVideo(TRange<FTimespan> TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample) override;
	virtual void FlushSamples() override;
	virtual void SetMinExpectedNextSequenceIndex(TOptional<int32> InNextSequenceIndex) override;
	virtual EFetchBestSampleResult FetchBestVideoSampleForTimeRange(const TRange<FMediaTimeStamp>& TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample, bool bReverse, bool bConsistentResult) override;
	virtual bool PeekVideoSampleTime(FMediaTimeStamp & TimeStamp) override;
	virtual bool DiscardVideoSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse) override;
	//~ End IMediaSamples

	//~ Begin IMediaTracks
	virtual bool GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const override;
	virtual int32 GetNumTracks(EMediaTrackType TrackType) const override;
	virtual int32 GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual int32 GetSelectedTrack(EMediaTrackType TrackType) const override;
	virtual FText GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual int32 GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual FString GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual FString GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual bool GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const override;
	virtual bool SelectTrack(EMediaTrackType TrackType, int32 TrackIndex) override;
	virtual bool SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex) override;
	//~ End IMediaTracks

private:
	/** Subscribe to FMediaTextureTracker's register/unregister delegates so providers added after Open() are picked up. Idempotent. */
	void AddMediaTextureTrackerEventHandlers();

	/** Drop the tracker delegate handles. Called from Close() and the destructor. Idempotent. */
	void RemoveMediaTextureTrackerEventHandlers();

	/** Re-register providers from the tracker objects bound to MediaTextureWeakPtr. Idempotent. */
	void RebuildTileVisibilityProviders();

	/** The duration of the currently loaded media. */
	FTimespan CurrentDuration;

	/** The current playback rate. */
	float CurrentRate;

	/** The last playback rate not zero (zero if never played). */
	float LastNonZeroRate;

	/** The player's current state. */
	EMediaState CurrentState;

	/** The current time of the playback. */
	FTimespan CurrentTime;

	/** Current seek index portion of sequence index */
	int32 CurrentSeekIndex;

	/** The URL of the currently opened media. */
	FString CurrentUrl;

	/** Stable per-facade identifier pushed down by FMediaPlayerFacade::SetGuid; used in logs to correlate across the facade and loader/reader layers. */
	FGuid PlayerGuid;

	/** Whether an offset to delta time has been applied yet. */
	bool DeltaTimeHackApplied;

	/** The media event handler. */
	IMediaEventSink& EventSink;

	/** Sample time of the last fetched video sample. */
	FTimespan LastFetchTime;

	/** The image sequence loader. */
	TSharedPtr<FImgMediaLoader, ESPMode::ThreadSafe> Loader;

	/** If playback just restarted from the Stopped state. */
	bool PlaybackRestarted;

	/** The scheduler for image loading. */
	TSharedPtr<FImgMediaScheduler, ESPMode::ThreadSafe> Scheduler;

	/** Index of the selected video track. */
	int32 SelectedVideoTrack;

	/** Should the video loop to the beginning at completion */
    bool ShouldLoop;

	/** The global cache to use. */
	TSharedPtr<FImgMediaGlobalCache, ESPMode::ThreadSafe> GlobalCache;

	/** True if we have run RequestFrame already for this frame. */
	bool RequestFrameHasRun;

	/** True if facade has signaled it uses blocking playback, false if not */
	bool PlaybackIsBlocking;

	/** Weak pointer to the active media texture. */
	TWeakObjectPtr<UMediaTexture> MediaTextureWeakPtr;

	/** Strong refs to provider instances built by RebuildTileVisibilityProviders; the resolver holds them weakly. */
	TArray<TSharedPtr<IMediaTileVisibilityProvider, ESPMode::ThreadSafe>> OwnedTileVisibilityProviders;

	/** FMediaTextureTracker delegate handles - subscribed in Open, removed in Close. */
	FDelegateHandle TrackerObjectRegisteredHandle;
	FDelegateHandle TrackerObjectUnregisteredHandle;
};

#undef UE_API