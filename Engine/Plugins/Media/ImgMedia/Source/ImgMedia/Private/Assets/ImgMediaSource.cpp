// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImgMediaSource.h"

#include "GameFramework/Actor.h"
#include "HAL/FileManager.h"
#include "IImgMediaModule.h"
#include "ImgMediaGlobalCache.h"
#include "ImgMediaPrivate.h"
#include "MediaSourcePathValidation.h"
#include "MediaPlayer.h"
#include "Misc/Paths.h"
#include "UObject/UE5MainStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImgMediaSource)

namespace UE::ImgMediaSource::Private
{
	/**
	 * Returns list of possible sequence base paths in order of priority.
	 */
	const TArray<FString>& GetPossibleBasePaths()
	{
		static const TArray<FString> PossibleBasePaths
		{
			FPaths::ProjectContentDir(),
			FPaths::ProjectDir()
		};
		return PossibleBasePaths;
	}

	/**
	 * Given a full path, find under which base path it could be.
	 * returns an empty string if it is not under any base dir.
	 */
	FString FindFullBasePath(const FString& InFullPath)
	{
		for (const FString& BasePath : GetPossibleBasePaths())
		{
			const FString FullBasePath = FPaths::ConvertRelativePathToFull(BasePath);
			if (FPaths::IsUnderDirectory(InFullPath, FullBasePath))
			{
				return FullBasePath;
			}
		}
		return FString();
	}
}

/* UImgMediaSource structors
 *****************************************************************************/

UImgMediaSource::UImgMediaSource()
	: IsPathRelativeToProjectRoot_DEPRECATED(false)
	, FrameRateOverride(0, 0)
	, LayerName(FString())
	, bFillGapsInSequence(true)
	, NativeSourceColorSettings(MakeShared<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe>())
{
}


/* UImgMediaSource interface
 *****************************************************************************/

void UImgMediaSource::GetProxies(TArray<FString>& OutProxies) const
{
	IFileManager::Get().FindFiles(OutProxies, *FPaths::Combine(GetFullPath(), TEXT("*")), false, true);
}

const FString UImgMediaSource::GetSequencePath() const
{
	return ExpandSequencePathTokens(SequencePath.Path);
}

void UImgMediaSource::SetSequencePath(const FString& Path)
{
	SequencePath.Path = SanitizeTokenizedSequencePath(Path);
}

void UImgMediaSource::SetTokenizedSequencePath(const FString& Path)
{
	SequencePath.Path = SanitizeTokenizedSequencePath(Path);
}

FString UImgMediaSource::ExpandSequencePathTokens(const FString& InPath)
{
	return InPath
		.Replace(TEXT("{engine_dir}"), *FPaths::ConvertRelativePathToFull(FPaths::EngineDir()))
		.Replace(TEXT("{project_dir}"), *FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()))
		;
}

FString UImgMediaSource::SanitizeTokenizedSequencePath(const FString& InPath)
{
	FString SanitizedPath = InPath.TrimStartAndEnd().Replace(TEXT("\""), TEXT(""));
	FPaths::NormalizeDirectoryName(SanitizedPath);

	if (SanitizedPath.IsEmpty())
	{
		return SanitizedPath;
	}

	// Replace supported tokens
	FString ExpandedPath = ExpandSequencePathTokens(SanitizedPath);

	FString SelectedRootPath;

	// Expand relative path with the possible base paths.
	if (FPaths::IsRelative(ExpandedPath))
	{
		// It could be a path relative to the process BaseDir
		FString FullExpandedPath = FPaths::ConvertRelativePathToFull(ExpandedPath);
		if (FPaths::DirectoryExists(FullExpandedPath) || FPaths::FileExists(FullExpandedPath))
		{
			ExpandedPath = FullExpandedPath;
			SelectedRootPath = UE::ImgMediaSource::Private::FindFullBasePath(FullExpandedPath);
		}
		else
		{
			// If it is not relative to BaseDir, we try the other possible bases.
			for (const FString& BasePath : UE::ImgMediaSource::Private::GetPossibleBasePaths())
			{
				const FString FullBasePath = FPaths::ConvertRelativePathToFull(BasePath);
				FullExpandedPath = FPaths::ConvertRelativePathToFull(FullBasePath, ExpandedPath);
				// Note: Directory or file needs to exist to figure it out.
				if (FPaths::DirectoryExists(FullExpandedPath) || FPaths::FileExists(FullExpandedPath))
				{
					ExpandedPath = FullExpandedPath;
					SelectedRootPath = FullBasePath;
					break;
				}
			}
		}
	}
	else
	{
		// For an absolute path, we still need to find which base it is under.
		SelectedRootPath = UE::ImgMediaSource::Private::FindFullBasePath(ExpandedPath);
	}

	// Re-check after expansion: tokens or normalization can produce a UNC from a clean input.
	if (!UE::MediaAssets::IsSourcePathAllowed(ExpandedPath))
	{
		UE_LOGF(LogImgMedia, Warning, "Rejecting non-local sequence path: %ls", *InPath);
		return FString();
	}

	// Chop trailing file path, in case the user picked a file instead of a folder
	if (FPaths::FileExists(ExpandedPath))
	{
		ExpandedPath = FPaths::GetPath(ExpandedPath);
		SanitizedPath = FPaths::GetPath(SanitizedPath);
	}

	// If the user picked the absolute path of a directory that is inside the project, use relative path.
	// Unless the user has a token in the beginning.
	if (!InPath.Len() || InPath[0] != '{') // '{' indicates that the path begins with a token
	{
		if (!SelectedRootPath.IsEmpty())
		{
			FString PathRelativeToProject;

			if (IsPathUnderBasePath(ExpandedPath, SelectedRootPath, PathRelativeToProject))
			{
				// Sanitized relative path expected to start with "./"
				if (!PathRelativeToProject.StartsWith(TEXT("./")))
				{
					PathRelativeToProject = FPaths::Combine(TEXT("."), PathRelativeToProject);
				}
				SanitizedPath = PathRelativeToProject;
			}
		}
		else
		{
			// the path was not inside the project, return absolute path.
			SanitizedPath = ExpandedPath;
		}
	}

	return SanitizedPath;
}

void UImgMediaSource::AddTargetObject(AActor* InActor)
{
	// Runtime breadcrumb for unmigrated external callers - the deprecation marker is easy to
	// miss and the function silently does nothing, so a tile-visibility regression would
	// otherwise surface only as degraded mip selection at playback time.
	UE_LOG(LogImgMedia, Warning,
		TEXT("UImgMediaSource::AddTargetObject is deprecated and is now a no-op. ")
		TEXT("Tile-visibility consumers should register through FMediaTextureTracker. Actor: %s"),
		*GetNameSafe(InActor));
}


void UImgMediaSource::RemoveTargetObject(AActor* InActor)
{
	UE_LOG(LogImgMedia, Warning,
		TEXT("UImgMediaSource::RemoveTargetObject is deprecated and is now a no-op. ")
		TEXT("Tile-visibility consumers should register through FMediaTextureTracker. Actor: %s"),
		*GetNameSafe(InActor));
}


/* IMediaOptions interface
 *****************************************************************************/

bool UImgMediaSource::GetMediaOption(const FName& Key, bool DefaultValue) const
{
	if (Key == ImgMedia::FillGapsInSequenceOption)
	{
		return bFillGapsInSequence;
	}

	return Super::GetMediaOption(Key, DefaultValue);
}

int64 UImgMediaSource::GetMediaOption(const FName& Key, int64 DefaultValue) const
{
	if (Key == ImgMedia::FrameRateOverrideDenonimatorOption)
	{
		return FrameRateOverride.Denominator;
	}

	if (Key == ImgMedia::FrameRateOverrideNumeratorOption)
	{
		return FrameRateOverride.Numerator;
	}

	return Super::GetMediaOption(Key, DefaultValue);
}


FString UImgMediaSource::GetMediaOption(const FName& Key, const FString& DefaultValue) const
{
	if (Key == ImgMedia::ProxyOverrideOption)
	{
		return ProxyOverride;
	}

	if (Key == ImgMedia::LayerNameOption)
	{
		return LayerName;
	}

	if (Key == UMediaPlayer::MediaInfoNameStartTimecodeValue.Resolve())
	{
		return StartTimecode.ToString();
	}

	return Super::GetMediaOption(Key, DefaultValue);
}

TSharedPtr<IMediaOptions::FDataContainer, ESPMode::ThreadSafe> UImgMediaSource::GetMediaOption(const FName& Key, const TSharedPtr<FDataContainer, ESPMode::ThreadSafe>& DefaultValue) const
{
	if (Key == ImgMedia::SourceColorSettingsOption)
	{
		NativeSourceColorSettings->Update(SourceColorSettings);

		return NativeSourceColorSettings;
	}

	return Super::GetMediaOption(Key, DefaultValue);
}

bool UImgMediaSource::HasMediaOption(const FName& Key) const
{
	if ((Key == ImgMedia::FillGapsInSequenceOption) ||
		(Key == ImgMedia::FrameRateOverrideDenonimatorOption) ||
		(Key == ImgMedia::FrameRateOverrideNumeratorOption) ||
		(Key == ImgMedia::ProxyOverrideOption) ||
		(Key == ImgMedia::LayerNameOption) ||
		(Key == ImgMedia::SourceColorSettingsOption))
	{
		return true;
	}

	return Super::HasMediaOption(Key);
}


/* UMediaSource interface
 *****************************************************************************/

FString UImgMediaSource::GetUrl() const
{
	return FString(TEXT("img://")) + GetFullPath();
}


bool UImgMediaSource::Validate() const
{
	return FPaths::DirectoryExists(GetFullPath());
}

#if WITH_EDITOR
void UImgMediaSource::GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const
{
	OutInfoElements.Add(FInfoElement(
		NSLOCTEXT("ImgMediaSource", "MediaConfigurationHeader", "Media Configuration"),
		NSLOCTEXT("ImgMediaSource", "FilePathLabel", "File Path"),
		FText::FromString(GetFullPath())));
}

void UImgMediaSource::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Has FillGapsInSequence changed?
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, bFillGapsInSequence))
	{
		// Clear the cache, as effectively the frames have changed.
		FImgMediaGlobalCache* GlobalCache = IImgMediaModule::GetGlobalCache();
		if (GlobalCache != nullptr)
		{
			GlobalCache->EmptyCache();
		}
	}
	else if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, SequencePath))
	{
		GenerateThumbnail();
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FMediaSourceColorSettings, ColorSpaceOverride))
	{
		SourceColorSettings.UpdateColorSpaceChromaticities();
	}

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, SourceColorSettings))
	{
		NativeSourceColorSettings->Update(SourceColorSettings);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // WITH_EDITOR

/* UFileMediaSource implementation
 *****************************************************************************/

FString UImgMediaSource::GetFullPath() const
{
	const FString ExpandedSequencePath = GetSequencePath();

	FString FullPath;
	if (FPaths::IsRelative(ExpandedSequencePath))
	{
		for (const FString& BasePath : UE::ImgMediaSource::Private::GetPossibleBasePaths())
		{
			const FString Candidate = FPaths::ConvertRelativePathToFull(FPaths::Combine(BasePath, ExpandedSequencePath));
			if (FPaths::DirectoryExists(Candidate))
			{
				FullPath = Candidate;
				break;
			}
		}

		// If we can't confirm because path doesn't exist, we will default to project dir for backward compatibility.
		if (FullPath.IsEmpty())
		{
			FullPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectDir(), ExpandedSequencePath));
		}
	}
	else
	{
		FullPath = ExpandedSequencePath;
	}

	// Re-check after resolution: token expansion can produce a UNC from a clean stored path.
	if (!UE::MediaAssets::IsSourcePathAllowed(FullPath))
	{
		UE_LOGF(LogImgMedia, Warning, "Rejecting non-local sequence path: %ls", *FullPath);
		return FString();
	}

	return FullPath;
}

void UImgMediaSource::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITOR
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::ImgMediaPathResolutionWithEngineOrProjectTokens)
	{
		if (Ar.IsLoading() && !IsPathRelativeToProjectRoot_DEPRECATED)
		{
			// This is an object that was saved with the old value (or before the property was added), so we need to convert the path accordingly

			IsPathRelativeToProjectRoot_DEPRECATED = true;

			if (FPaths::IsRelative(SequencePath.Path))
			{
				SequencePath.Path = SanitizeTokenizedSequencePath(SequencePath.Path);
			}
		}
	}
#endif
}

bool UImgMediaSource::IsPathUnderBasePath(const FString& InPath, const FString& InBasePath, FString& OutRelativePath)
{
	OutRelativePath = InPath;

	return 
		FPaths::MakePathRelativeTo(OutRelativePath, *InBasePath) 
		&& !OutRelativePath.StartsWith(TEXT(".."));
}

