// Copyright Epic Games, Inc. All Rights Reserved.

#include "Assets/ImgMediaTileVisibilityResolver.h"

#include "Engine/Engine.h"
#include "HAL/IConsoleManager.h"

// Console variables consumed by the visibility providers (looked up via FindConsoleVariable
// rather than referenced directly so the providers don't have to share a private header).
static TAutoConsoleVariable<float> CVarImgMediaMipLevelPadding(
	TEXT("ImgMedia.MipMapLevelPadding"),
	0.0f,
	TEXT("Value padded onto the estimated (minimum and maximum) mipmap levels used by the loader.\n"),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarImgMediaEnableMipMapTileDebugDraw(
	TEXT("ImgMedia.EnableMipMapTileDebugDraw"),
	false,
	TEXT("Enables drawing of the bounding volume around visible tiles for trouble shooting tile update issues.\n")
	TEXT("   0: off (default)\n")
	TEXT("   1: on\n"),
	ECVF_Default);

/** Time spent gathering tile visibility from registered providers. */
DECLARE_CYCLE_STAT(TEXT("ImgMedia Visibility Resolver"), STAT_ImgMedia_VisibilityResolver, STATGROUP_Media);

FImgMediaTileVisibilityResolver::FImgMediaTileVisibilityResolver() = default;

FImgMediaTileVisibilityResolver::~FImgMediaTileVisibilityResolver()
{
	for (FProviderEntry& Entry : Providers)
	{
		UnsubscribeFromProvider(Entry);
	}
}

void FImgMediaTileVisibilityResolver::UnsubscribeFromProvider(FProviderEntry& InOutEntry)
{
	if (!InOutEntry.OnInputsChangedHandle.IsValid())
	{
		return;
	}
	if (TSharedPtr<IMediaTileVisibilityProvider> Pinned = InOutEntry.Weak.Pin())
	{
		Pinned->RemoveOnInputsChangedHandler(InOutEntry.OnInputsChangedHandle);
	}
	InOutEntry.OnInputsChangedHandle.Reset();
}

void FImgMediaTileVisibilityResolver::SetSequenceInfo(const FMediaSequenceInfo& InInfo)
{
	FScopeLock Lock(&Mutex);
	SequenceInfo = InInfo;
	CachedSnapshot.Reset();
}

FMediaSequenceInfo FImgMediaTileVisibilityResolver::GetSequenceInfo() const
{
	FScopeLock Lock(&Mutex);
	return SequenceInfo;
}

void FImgMediaTileVisibilityResolver::RegisterProvider(TSharedRef<IMediaTileVisibilityProvider> InProvider)
{
	FScopeLock Lock(&Mutex);

	const bool bAlreadyRegistered = Providers.ContainsByPredicate(
		[&InProvider](const FProviderEntry& Entry)
		{
			TSharedPtr<IMediaTileVisibilityProvider> P = Entry.Weak.Pin();
			return P.IsValid() && P == InProvider;
		});

	if (!bAlreadyRegistered)
	{
		FProviderEntry NewEntry;
		NewEntry.Weak = TWeakPtr<IMediaTileVisibilityProvider>(InProvider);
		// Provider state changes invalidate the snapshot so the next GetSnapshot rebuilds with the new
		// inputs. TickFrame's per-frame Update then compares against MipTilesPresent and queues missing
		// tiles - which is what ultimately drives the paused-refresh path via NotifyWorkComplete.
		NewEntry.OnInputsChangedHandle = InProvider->AddOnInputsChangedHandler(
			FSimpleDelegate::CreateRaw(this, &FImgMediaTileVisibilityResolver::InvalidateSnapshot));
		Providers.Add(MoveTemp(NewEntry));
		CachedSnapshot.Reset();
	}
}

void FImgMediaTileVisibilityResolver::UnregisterProvider(const TSharedRef<IMediaTileVisibilityProvider>& InProvider)
{
	FScopeLock Lock(&Mutex);
	Providers.RemoveAll([&InProvider](FProviderEntry& Entry)
	{
		TSharedPtr<IMediaTileVisibilityProvider> P = Entry.Weak.Pin();
		if (!P.IsValid() || P == InProvider)
		{
			UnsubscribeFromProvider(Entry);
			return true;
		}
		return false;
	});
	CachedSnapshot.Reset();
}

void FImgMediaTileVisibilityResolver::ClearProviders()
{
	FScopeLock Lock(&Mutex);
	for (FProviderEntry& Entry : Providers)
	{
		UnsubscribeFromProvider(Entry);
	}
	Providers.Reset();
	CachedSnapshot.Reset();
}

int32 FImgMediaTileVisibilityResolver::NumProviders() const
{
	FScopeLock Lock(&Mutex);
	int32 Count = 0;
	for (const FProviderEntry& Entry : Providers)
	{
		TSharedPtr<IMediaTileVisibilityProvider> P = Entry.Weak.Pin();
		if (P.IsValid() && P->IsAlive())
		{
			++Count;
		}
	}
	return Count;
}

bool FImgMediaTileVisibilityResolver::HasProviders() const
{
	return NumProviders() > 0;
}

void FImgMediaTileVisibilityResolver::InvalidateSnapshot()
{
	FScopeLock Lock(&Mutex);
	CachedSnapshot.Reset();
}

void FImgMediaTileVisibilityResolver::AddVisibilitySnapshotDebugOverlay()
{
	if (GEngine == nullptr)
	{
		return;
	}

	// Remark: avoiding holding lock while calling AddOnScreenDebugMessage.
	const TSharedPtr<FMediaTileVisibilityRequest, ESPMode::ThreadSafe> Snapshot = GetSnapshot();
	if (!Snapshot)
	{
		return;
	}
	
	const FString SequenceName = GetSequenceInfo().Name.ToString();

	TArray<int32> VisibleMips;
	int32 TotalVisibleTiles = 0;
	VisibleMips.Reserve(Snapshot->VisibleTiles.Num());
	for (const TPair<int32, FMediaTileSelection>& Pair : Snapshot->VisibleTiles)
	{
		VisibleMips.Add(Pair.Key);
		TotalVisibleTiles += Pair.Value.NumVisibleTiles();
	}

	if (VisibleMips.Num() > 0)
	{
		VisibleMips.Sort();
		const FString MipList = FString::JoinBy(VisibleMips, TEXT(", "), [](int32 Mip) { return FString::FromInt(Mip); });

		// Stable per-resolver keys make each line replace the previous frame's instead of stacking,
		// and a TTL longer than one frame absorbs an occasional missed end-frame.
		const uint64 BaseKey = static_cast<uint64>(GetTypeHash(this)) << 1;
		constexpr float DebugMessageTtl = 0.5f;
		GEngine->AddOnScreenDebugMessage(BaseKey | 0, DebugMessageTtl, FColor::Yellow,
			FString::Printf(TEXT("%s Mip Level(s): [%s]"), *SequenceName, *MipList));
		GEngine->AddOnScreenDebugMessage(BaseKey | 1, DebugMessageTtl, FColor::Yellow,
			FString::Printf(TEXT("%s Num Tile(s): %d"), *SequenceName, TotalVisibleTiles));
	}
}

TSharedPtr<FMediaTileVisibilityRequest, ESPMode::ThreadSafe> FImgMediaTileVisibilityResolver::GetSnapshot()
{
	SCOPE_CYCLE_COUNTER(STAT_ImgMedia_VisibilityResolver);

	// Phase 1: fast path under lock.
	{
		FScopeLock Lock(&Mutex);
		if (CachedSnapshot.IsValid())
		{
			return CachedSnapshot;
		}
	}

	// Phase 2: snapshot inputs under lock, then drop it.
	// Snapshot inputs under the lock so we can run provider work outside it.
	// Providers may take other locks (e.g. ImgMediaSceneViewExtension::ViewInfosMutex)
	// that are acquired in the opposite order on the game thread, which would
	// deadlock if we held Mutex across GatherVisibleTiles.
	FMediaSequenceInfo LocalSequenceInfo;
	TArray<TSharedPtr<IMediaTileVisibilityProvider>> LiveProviders;
	{
		FScopeLock Lock(&Mutex);
		if (CachedSnapshot.IsValid()) // re-check; another thread may have built
		{
			return CachedSnapshot;
		}
		LocalSequenceInfo = SequenceInfo;
		LiveProviders.Reserve(Providers.Num());
		for (auto It = Providers.CreateIterator(); It; ++It)
		{
			TSharedPtr<IMediaTileVisibilityProvider> Provider = It->Weak.Pin();
			if (Provider.IsValid() && Provider->IsAlive())
			{
				LiveProviders.Add(MoveTemp(Provider));
			}
			else
			{
				// Provider already gone - the OnInputsChanged subscription went with it; just
				// drop the bookkeeping entry. No broadcast: a dead consumer cannot be observed.
				UnsubscribeFromProvider(*It);
				It.RemoveCurrent();
			}
		}
	}

	// Phase 3: build outside the lock — provider work is free to take any lock.
	TSharedPtr<FMediaTileVisibilityRequest, ESPMode::ThreadSafe> NewSnapshot
		= MakeShared<FMediaTileVisibilityRequest, ESPMode::ThreadSafe>();
	for (const TSharedPtr<IMediaTileVisibilityProvider>& Provider : LiveProviders)
	{
		FMediaTileVisibilityRequest Local;
		Provider->GatherVisibleTiles(LocalSequenceInfo, Local);
		NewSnapshot->Merge(Local);
	}

	// Phase 4: publish atomically. First-writer-wins so we never replace a
	// snapshot a concurrent caller has already returned. If an Invalidate /
	// Register / Unregister / Clear / Broadcast races in between Phase 2 and
	// here, we may cache a snapshot built from inputs that were already
	// invalidated mid-build. FImgMediaLoader::TickFrame() invalidates each
	// game tick, so a stale publish self-heals within one frame.
	FScopeLock Lock(&Mutex);
	if (!CachedSnapshot.IsValid())
	{
		CachedSnapshot = NewSnapshot;
	}
	return CachedSnapshot;
}

int32 FImgMediaTileVisibilityResolver::GetMinimumMipLevelToUpscale()
{
	const TSharedPtr<FMediaTileVisibilityRequest, ESPMode::ThreadSafe> Snapshot = GetSnapshot();
	return Snapshot.IsValid() ? Snapshot->MipLevelToUpscale : -1;
}
