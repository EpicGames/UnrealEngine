// Copyright Epic Games, Inc. All Rights Reserved.

#include "Assets/Providers/ImgMediaPlaneVisibilityProvider.h"

#include "Assets/MediaSequenceInfo.h"
#include "Assets/Providers/ImgMediaProviderUtils.h"
#include "Async/Async.h"
#include "Components/MeshComponent.h"
#include "ConvexVolume.h"
#include "Engine/Engine.h"
#include "ImgMediaPrivate.h"
#include "ImgMediaSceneViewExtension.h"
#include "Math/UnrealMathUtility.h"
#include "UObject/GarbageCollection.h"

#if WITH_EDITOR
#include "DrawDebugHelpers.h"
#include "Editor.h"
#endif

namespace ImgMediaPlaneProvider
{
	// Plane-specific corner mip cache helpers. The shared CVar accessors and
	// CalculateMipLevelAniso live in ImgMediaProviderUtils.h.
	FORCEINLINE bool GetCachedMipLevel(const TArray<float>& Cache, int32 Address0X, int32 Address0Y, int32 RowSize, float& OutCalculatedLevel)
	{
		const int32 Index = Address0Y * RowSize + Address0X;
		if (Cache[Index] >= 0.0f)
		{
			OutCalculatedLevel = Cache[Index];
			return true;
		}
		return false;
	}

	FORCEINLINE void SetCachedMipLevel(TArray<float>& Cache, int32 Address0X, int32 Address0Y, int32 RowSize, float InCalculatedLevel)
	{
		Cache[Address0Y * RowSize + Address0X] = InCalculatedLevel;
	}

	FORCEINLINE void ResetMipLevelCache(TArray<float>& Cache)
	{
		for (float& Level : Cache)
		{
			Level = -1.0f;
		}
	}
} // namespace ImgMediaPlaneProvider

FImgMediaPlaneVisibilityProvider::FImgMediaPlaneVisibilityProvider(
	const FImgMediaPlaneVisibilityProviderParams& InParams)
	: Params(InParams)
	, PlaneSize(FVector::ZeroVector)
{
	if (UMeshComponent* Mesh = Params.MeshComponent.Get())
	{
		PlaneSize = 2.0f * Mesh->CalcLocalBounds().BoxExtent;
	}
	else
	{
		UE_LOGF(LogImgMedia, Error,
			"FImgMediaPlaneVisibilityProvider is missing its plane mesh component.");
	}
}

bool FImgMediaPlaneVisibilityProvider::IsAlive() const
{
	return Params.MeshComponent.IsValid();
}

void FImgMediaPlaneVisibilityProvider::GatherVisibleTiles(
	const FMediaSequenceInfo& InSequenceInfo,
	FMediaTileVisibilityRequest& OutRequest) const
{
	using namespace ImgMediaPlaneProvider;
	using namespace UE::ImgMediaProviders;

	// Snapshot UObject-derived values then drop the GC guard; the math loop below
	// runs on plain values, and holding the guard across a frustum walk would stall GC.
	FTransform MeshTransform;
	FVector MeshScale;
	FPrimitiveComponentId MeshPrimitiveSceneId;
	{
		FGCScopeGuard GCGuard;
		UMeshComponent* Mesh = Params.MeshComponent.Get();
		if (Mesh == nullptr || !Mesh->ShouldRender())
		{
			return;
		}
		MeshTransform = Mesh->GetComponentTransform();
		MeshScale = Mesh->GetComponentScale();
		MeshPrimitiveSceneId = Mesh->GetPrimitiveSceneId();
	}

	TSharedPtr<FImgMediaSceneViewExtension, ESPMode::ThreadSafe> SVE = Params.SceneViewExtension.Pin();
	if (!SVE.IsValid())
	{
		return;
	}

	// Select the appropriate set(s) of cached view infos based on the target resolution mask.
	TArray<FImgMediaViewInfo> ViewInfos = SVE->GetViewInfos(Params.TargetViewResolutionMask);

	if (ViewInfos.IsEmpty())
	{
		return;
	}

	// Texel-offset and per-tile mip math divides by the sequence pixel dimensions.
	if (InSequenceInfo.Dim.X <= 0 || InSequenceInfo.Dim.Y <= 0)
	{
		return;
	}

	// Plane provider always advertises its configured upscale level; resolver merges across providers.
	if (Params.MipLevelToUpscale >= 0)
	{
		OutRequest.MipLevelToUpscale = Params.MipLevelToUpscale;
	}

	const float MipMapBias = Params.MipMapLODBias;
	const float MipMapLevelPadding = GetMipLevelPadding();
#if WITH_EDITOR
	const bool bDrawDebugSpheres = GetDebugDrawEnabled();
#endif
	const FIntPoint& SequenceTileNum = InSequenceInfo.TilingDescription.TileNum;
	const int32 CornerCacheRowSize = SequenceTileNum.X + 1;

	// Stack-local: concurrent gathers don't share state. Reset per view in the loop.
	TArray<float> CornerMipLevelsCached;
	CornerMipLevelsCached.SetNum(CornerCacheRowSize * (SequenceTileNum.Y + 1));

	FVector PlaneCornerWS = MeshTransform.TransformPosition(FVector(0, -0.5f * PlaneSize.Y, 0.5f * PlaneSize.Z));
	FVector DirXWS = MeshTransform.TransformVector(FVector(0, PlaneSize.Y, 0));
	FVector DirYWS = MeshTransform.TransformVector(FVector(0, 0, -PlaneSize.Z));
	FVector TexelOffsetXWS = MeshTransform.TransformVector(FVector(0, PlaneSize.Y / InSequenceInfo.Dim.X, 0));
	FVector TexelOffsetYWS = MeshTransform.TransformVector(FVector(0, 0, -PlaneSize.Z / InSequenceInfo.Dim.Y));

#if WITH_EDITOR
	// Shared center/radius for the above-cap debug spheres; radii are offset per mip
	// below so a stack of them reads as concentric rings instead of one solid sphere.
	const FVector PlateCenterWS = MeshTransform.TransformPosition(FVector::ZeroVector);
	const float PlateBoundingRadiusWS = 0.5f * (float)FMath::Sqrt(
		2.0 * FMath::Square((PlaneSize * MeshScale).GetAbsMax()));

	// Debug spheres batched into one TaskGraph dispatch.
	struct FPendingDebugSphere
	{
		FVector Center;
		float Radius;
		FColor Color;
		int32 Segments;
	};
	TArray<FPendingDebugSphere> PendingDebugSpheres;
#endif

	for (const FImgMediaViewInfo& ViewInfo : ViewInfos)
	{
		if (IsPrimitiveComponentHidden(MeshPrimitiveSceneId, ViewInfo))
		{
			continue;
		}

		ResetMipLevelCache(CornerMipLevelsCached);

		// Get frustum.
		FConvexVolume ViewFrustum;
		GetViewFrustumBounds(ViewFrustum, ViewInfo.OverscanViewProjectionMatrix, false, false);

		// Cap the quadtree at the tile-pyramid root; above it, GetPartialTileNum() drops
		// below 1 and StepX/StepY walk the tile center off the plate. MipLevelRange stays
		// clamped to NumMipsMax so the high-mip emission below still sees the actual mip.
		const int32 NumMipsMax = FMath::Max(0, InSequenceInfo.NumMipLevels - 1);
		const int32 MaxLevel = ComputeQuadtreeMaxMipLevel(
			InSequenceInfo.NumMipLevels, SequenceTileNum, InSequenceInfo.IsTiled());
		int MipLevelDiv = 1 << MaxLevel;

		FIntPoint CurrentNumTiles = FIntPoint(1, 1);

		if (InSequenceInfo.IsTiled())
		{
			CurrentNumTiles.X = FMath::CeilToInt(float(InSequenceInfo.Dim.X) / MipLevelDiv / InSequenceInfo.TilingDescription.TileSize.X);
			CurrentNumTiles.Y = FMath::CeilToInt(float(InSequenceInfo.Dim.Y) / MipLevelDiv / InSequenceInfo.TilingDescription.TileSize.Y);
		}

		// Quadtree stack. Output is order-independent so DFS via TArray::Pop is equivalent
		// to BFS without TQueue's per-node alloc.
		TArray<FIntVector> Tiles;
		Tiles.Reserve(CurrentNumTiles.X * CurrentNumTiles.Y);
		for (int32 TileY = 0; TileY < CurrentNumTiles.Y; ++TileY)
		{
			for (int32 TileX = 0; TileX < CurrentNumTiles.X; ++TileX)
			{
				Tiles.Emplace(TileX, TileY, MaxLevel);
			}
		}

		while (!Tiles.IsEmpty())
		{
			FIntVector Tile = Tiles.Pop(EAllowShrinking::No);

			int32 CurrentMipLevel = Tile.Z;
			MipLevelDiv = 1 << CurrentMipLevel;

			if (InSequenceInfo.IsTiled())
			{
				CurrentNumTiles.X = FMath::Max(1, FMath::CeilToInt((float(InSequenceInfo.Dim.X) / MipLevelDiv) / InSequenceInfo.TilingDescription.TileSize.X));
				CurrentNumTiles.Y = FMath::Max(1, FMath::CeilToInt((float(InSequenceInfo.Dim.Y) / MipLevelDiv) / InSequenceInfo.TilingDescription.TileSize.Y));
			}

			FVector2f CurrentPartialTileNum = InSequenceInfo.GetPartialTileNum(CurrentMipLevel);

			// Exclude subdivided tiles (enqueued below) that are not present (i.e. mipped sequences with odd number of tiles)
			if (Tile.X >= CurrentNumTiles.X || Tile.Y >= CurrentNumTiles.Y)
			{
				continue;
			}

			// Calculate the tile location in world-space
			float StepX = float(Tile.X + 0.5f) / CurrentPartialTileNum.X;
			float StepY = float(Tile.Y + 0.5f) / CurrentPartialTileNum.Y;
			FVector TileCenterWS = PlaneCornerWS + (DirXWS * StepX + DirYWS * StepY);

			// Calculate the tile radius in world space
			FVector TileSizeWS = (PlaneSize * MeshScale) / FVector(1, CurrentPartialTileNum.X, CurrentPartialTileNum.Y);
			float TileRadiusInWS = 0.5f * (float)FMath::Sqrt(2 * FMath::Square(TileSizeWS.GetAbsMax()));

			// Now we check if tile spherical bounds are in view.
			if (ViewFrustum.IntersectSphere(TileCenterWS, TileRadiusInWS))
			{
				// Calculate the visible mip level range over all tile corners.
				int32 NumVisibleCorners = 0;
				FIntVector2 MipLevelRange = FIntVector2(TNumericLimits<int32>::Max(), 0);
				for (int32 CornerY = 0; CornerY < 2; ++CornerY)
				{
					for (int32 CornerX = 0; CornerX < 2; ++CornerX)
					{
						float CalculatedLevel;
						int32 TileCornerX = Tile.X + CornerX;
						int32 TileCornerY = Tile.Y + CornerY;

						// First we query the cached corner mip levels.
						FIntPoint BaseLevelCorner;
						BaseLevelCorner.X = FMath::Clamp(TileCornerX << CurrentMipLevel, 0, SequenceTileNum.X);
						BaseLevelCorner.Y = FMath::Clamp(TileCornerY << CurrentMipLevel, 0, SequenceTileNum.Y);
						bool bValidLevel = GetCachedMipLevel(CornerMipLevelsCached, BaseLevelCorner.X, BaseLevelCorner.Y, CornerCacheRowSize, CalculatedLevel);

						// If not found, calculate and cache it.
						if (!bValidLevel)
						{
							float CornerStepX = TileCornerX / CurrentPartialTileNum.X;
							float CornerStepY = TileCornerY / CurrentPartialTileNum.Y;
							FVector CornersWS = PlaneCornerWS + (DirXWS * CornerStepX + DirYWS * CornerStepY);

							if (CalculateMipLevelAniso(ViewInfo, CornersWS, CornersWS + TexelOffsetXWS, CornersWS + TexelOffsetYWS, CalculatedLevel))
							{
								CalculatedLevel += MipMapBias + ViewInfo.MaterialTextureMipBias;

								SetCachedMipLevel(CornerMipLevelsCached, BaseLevelCorner.X, BaseLevelCorner.Y, CornerCacheRowSize, CalculatedLevel);
								bValidLevel = true;
							}
						}

						if (bValidLevel)
						{
							MipLevelRange[0] = FMath::Min(MipLevelRange[0], FMath::Clamp(FMath::FloorToInt32(CalculatedLevel - MipMapLevelPadding), 0, NumMipsMax));
							MipLevelRange[1] = FMath::Max(MipLevelRange[1], FMath::Clamp(FMath::CeilToInt32(CalculatedLevel + MipMapLevelPadding), 0, NumMipsMax));
							NumVisibleCorners++;
						}
					}
				}

				// As an approximation, we force the lowest mip to 0 if only some corners are behind camera.
				if (NumVisibleCorners > 0 && NumVisibleCorners < 4)
				{
					MipLevelRange[0] = 0;
				}

				// If the lowest (calculated) mip level is below our current mip level, enqueue all 4 sub-tiles for further processing.
				if (MipLevelRange[0] < CurrentMipLevel)
				{
					for (int32 SubY = 0; SubY < FMath::Min(SequenceTileNum.Y, 2); ++SubY)
					{
						for (int32 SubX = 0; SubX < FMath::Min(SequenceTileNum.X, 2); ++SubX)
						{
							Tiles.Emplace((Tile.X << 1) + SubX, (Tile.Y << 1) + SubY, CurrentMipLevel - 1);
						}
					}
				}

				// Mark visible only when CurrentMipLevel sits inside the calculated range; the
				// lower-bound check skips redundant cap-level entries when the tile is entirely
				// above the cap and the high-mip emission below already covers it.
				if (MipLevelRange[0] <= CurrentMipLevel && MipLevelRange[1] >= CurrentMipLevel)
				{
					if (!OutRequest.VisibleTiles.Contains(CurrentMipLevel))
					{
						OutRequest.VisibleTiles.Emplace(CurrentMipLevel, FMediaTileSelection(CurrentNumTiles.X, CurrentNumTiles.Y));
					}

					OutRequest.VisibleTiles[CurrentMipLevel].SetVisible(Tile.X, Tile.Y);
				}

				// Emit single-tile visibility for mips above the cap; without these, far views
				// have no loaded mip and render black.
				EmitVisibilityAboveQuadtreeCap(InSequenceInfo.Dim,
					InSequenceInfo.TilingDescription.TileSize, MaxLevel, MipLevelRange, OutRequest);
#if WITH_EDITOR
				if (bDrawDebugSpheres)
				{
					// Red per-tile sphere; gated to match the mark-visible condition above so the
					// debug view stays in sync with the emitted mip list.
					if (MipLevelRange[0] <= CurrentMipLevel && MipLevelRange[1] >= CurrentMipLevel)
					{
						PendingDebugSpheres.Add({TileCenterWS, TileRadiusInWS, FColor::Red, 8});
					}

					// Green concentric spheres - one per above-cap mip; nested radii visualize the count.
					if (MipLevelRange[1] > MaxLevel)
					{
						const int32 HighMipMin = FMath::Max(MaxLevel + 1, MipLevelRange[0]);
						for (int32 HighMip = HighMipMin; HighMip <= MipLevelRange[1]; ++HighMip)
						{
							const float SphereRadius = PlateBoundingRadiusWS *
								(1.0f + 0.05f * static_cast<float>(HighMip - HighMipMin));
							PendingDebugSpheres.Add({PlateCenterWS, SphereRadius, FColor::Orange, 12});
						}
					}
				}
#endif // WITH_EDITOR
			}
		}
	}

#if WITH_EDITOR
	if (!PendingDebugSpheres.IsEmpty())
	{
		Async(EAsyncExecution::TaskGraphMainThread, [Spheres = MoveTemp(PendingDebugSpheres)]()
		{
			if (const UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : GWorld)
			{
				for (const FPendingDebugSphere& Sphere : Spheres)
				{
					DrawDebugSphere(World, Sphere.Center, Sphere.Radius, Sphere.Segments, Sphere.Color, false, 0.05f);
				}
			}
		});
	}
#endif
}
