// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Assets/MediaTileVisibility.h"
#include "Assets/Providers/ImgMediaProviderUtils.h"
#include "Math/Vector.h"

/** Construction parameters for the plane visibility provider; no plane-specific fields today. */
struct FImgMediaPlaneVisibilityProviderParams : public FImgMediaVisibilityProviderParams
{
};

/**
 * Visibility provider for media displayed on a flat (plane) mesh.
 * Performs a quadtree-style frustum walk over the sequence's tile pyramid
 * and emits tile selections per visible mip level.
 */
class FImgMediaPlaneVisibilityProvider final : public FMediaTileVisibilityProvider
{
public:
	explicit FImgMediaPlaneVisibilityProvider(const FImgMediaPlaneVisibilityProviderParams& InParams);

	//~ Begin IMediaTileVisibilityProvider
	virtual void GatherVisibleTiles(const FMediaSequenceInfo& InSequenceInfo,
		FMediaTileVisibilityRequest& OutRequest) const override;
	virtual bool IsAlive() const override;
	//~ End IMediaTileVisibilityProvider

private:
	FImgMediaPlaneVisibilityProviderParams Params;

	/**
	 * Local-space mesh bounds captured at construction. Frozen by design - if the consumer's
	 * mesh is replaced or rebuilt at runtime, register a fresh provider rather than mutating
	 * this.
	 */
	FVector PlaneSize;
};
