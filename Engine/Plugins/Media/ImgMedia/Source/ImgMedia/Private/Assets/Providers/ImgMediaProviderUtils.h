// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Assets/MediaTileVisibility.h"
#include "HAL/IConsoleManager.h"
#include "ImgMediaSceneViewExtension.h"
#include "Math/UnrealMathUtility.h"
#include "MediaTextureTracker.h"
#include "PrimitiveSceneInfo.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class FImgMediaSceneViewExtension;
class UMeshComponent;

/**
 * Construction parameters common to every ImgMedia visibility provider implementation.
 * Per-provider param structs (FImgMediaPlaneVisibilityProviderParams etc.) inherit from
 * this and add only what they need on top.
 */
struct FImgMediaVisibilityProviderParams
{
	/** Mesh component the media is rendered on. World-space bounds drive tile mip selection. */
	TWeakObjectPtr<UMeshComponent> MeshComponent;

	/** Source of cached camera view infos. */
	TWeakPtr<FImgMediaSceneViewExtension, ESPMode::ThreadSafe> SceneViewExtension;

	/** LOD bias added to calculated mip level (negative biases toward higher quality). */
	float MipMapLODBias = 0.f;

	/** Mip level to upscale into lower quality mips, or -1 to disable. */
	int32 MipLevelToUpscale = -1;

	/** Bitwise mask for which set of cached view infos to consume. */
	EMediaTextureTargetViewResolution TargetViewResolutionMask =
		EMediaTextureTargetViewResolution::RenderResolution;
};

namespace UE::ImgMediaProviders
{

/**
 * Helpers shared by FImgMediaPlaneVisibilityProvider and FImgMediaSphereVisibilityProvider.
 * The CVars themselves are registered in ImgMediaTileVisibilityResolver.cpp; these accessors
 * just resolve them by name once and then read the live value on each call.
 */

/** Padding added to the calculated mip-level range. CVar: ImgMedia.MipMapLevelPadding. */
inline float GetMipLevelPadding()
{
	static IConsoleVariable* CVar =
		IConsoleManager::Get().FindConsoleVariable(TEXT("ImgMedia.MipMapLevelPadding"));
	return CVar ? FMath::Max(CVar->GetFloat(), 0.0f) : 0.0f;
}

#if WITH_EDITOR
/** Whether to draw per-tile debug spheres in the editor. CVar: ImgMedia.EnableMipMapTileDebugDraw. */
inline bool GetDebugDrawEnabled()
{
	static IConsoleVariable* CVar =
		IConsoleManager::Get().FindConsoleVariable(TEXT("ImgMedia.EnableMipMapTileDebugDraw"));
	return CVar ? CVar->GetBool() : false;
}
#endif

/**
 * Cap for quadtree-style tile traversals: the highest mip level at which the per-axis
 * tile count is still >= 1. NumMipLevels counts texture downsamples; the quadtree halves
 * tile counts per axis. Once 2^Mip exceeds min(NumTiles), GetPartialTileNum() drops below
 * 1 and per-tile placement walks off the plate. Non-tiled sequences are not capped (their
 * GetPartialTileNum is fixed at (1,1) so tile placement never degenerates).
 *
 * @param InNumMipLevels  Number of mip levels in the asset (>= 1 for normal sequences;
 *                        0 is tolerated and yields a cap of 0).
 * @param InTileNum       Mip-0 tile count per axis (FMediaTextureTilingDescription::TileNum).
 *                        Zero/negative components are treated as 1 to avoid FloorLog2(0).
 * @param bInIsTiled      Pass FMediaSequenceInfo::IsTiled(). When false the cap is just
 *                        NumMipLevels - 1 since there's no tile pyramid to degenerate.
 *
 * @return The highest mip level the quadtree may safely traverse, in [0, NumMipLevels-1].
 */
FORCEINLINE int32 ComputeQuadtreeMaxMipLevel(int32 InNumMipLevels, const FIntPoint& InTileNum, bool bInIsTiled)
{
	const int32 NumMipsMax = FMath::Max(0, InNumMipLevels - 1);
	if (!bInIsTiled)
	{
		return NumMipsMax;
	}
	const int32 MinTiles = FMath::Max(1, FMath::Min(InTileNum.X, InTileNum.Y));
	const int32 TileMaxLevel = FMath::FloorLog2(static_cast<uint32>(MinTiles));
	return FMath::Min(NumMipsMax, TileMaxLevel);
}

/**
 * Emit "all-tiles-visible" entries for every mip in [InMipLevelRange[0], InMipLevelRange[1]]
 * that lives above the quadtree cap (InMaxLevel). The quadtree only walks down to InMaxLevel;
 * mips above it have no quadtree node to mark them visible, so a far-camera view would leave
 * those mips out of the request entirely and the renderer would render black where it expects
 * a coarse mip. Idempotent under repeated calls: each call overwrites the entry with a freshly
 * computed all-visible selection, equivalent to bit-OR since dimensions are pinned by
 * InMipZeroResolution / InTileSize.
 *
 * @param InMipZeroResolution  Pixel dimensions of the sequence at mip 0
 *                             (FMediaSequenceInfo::Dim). Used to compute per-mip tile counts
 *                             via FMediaTileSelection::CreateForTargetMipLevel.
 * @param InTileSize           Tile dimensions in pixels
 *                             (FMediaTextureTilingDescription::TileSize).
 * @param InMaxLevel           Quadtree cap (typically from ComputeQuadtreeMaxMipLevel).
 *                             Mips at or below this level are the quadtree's responsibility
 *                             and are left untouched.
 * @param InMipLevelRange      Calculated mip-level range [floor, ceil] for the tile being
 *                             processed. Must satisfy [0] <= [1]. When [1] <= InMaxLevel
 *                             the call is a no-op.
 * @param OutRequest           [in,out] Visibility request to extend in-place. Any existing
 *                             entry at the same mip is replaced by the all-visible selection.
 */
FORCEINLINE void EmitVisibilityAboveQuadtreeCap(
	const FIntPoint& InMipZeroResolution,
	const FIntPoint& InTileSize,
	int32 InMaxLevel,
	const FIntVector2& InMipLevelRange,
	FMediaTileVisibilityRequest& OutRequest)
{
	if (InMipLevelRange[1] <= InMaxLevel)
	{
		return;
	}
	const int32 HighMipMin = FMath::Max(InMaxLevel + 1, InMipLevelRange[0]);
	for (int32 HighMip = HighMipMin; HighMip <= InMipLevelRange[1]; ++HighMip)
	{
		OutRequest.VisibleTiles.Emplace(HighMip, FMediaTileSelection::CreateForTargetMipLevel(
			InMipZeroResolution, InTileSize, HighMip, /*bInDefaultVisibility=*/true));
	}
}

/** Inverts the show-only / hidden semantics of FImgMediaViewInfo into a single hidden test. */
inline bool IsPrimitiveComponentHidden(FPrimitiveComponentId ComponentId, const FImgMediaViewInfo& ViewInfo)
{
	const bool bIsPrimitiveContained = ViewInfo.PrimitiveComponentIds.Contains(ComponentId);
	return ViewInfo.bPrimitiveHiddenMode ? bIsPrimitiveContained : !bIsPrimitiveContained;
}

/** Minimalized version of FSceneView::ProjectWorldToScreen - returns false when behind the camera. */
FORCEINLINE bool ProjectWorldToScreenFast(const FVector& WorldPosition, const FIntRect& ViewRect,
	const FMatrix& ViewProjectionMatrix, FVector2D& out_ScreenPos)
{
	const FPlane Result = ViewProjectionMatrix.TransformFVector4(FVector4(WorldPosition, 1.f));
	if (Result.W > 0.0f)
	{
		const float NormalizedX = (Result.X / (Result.W * 2.f)) + 0.5f;
		const float NormalizedY = 1.f - (Result.Y / (Result.W * 2.f)) - 0.5f;
		out_ScreenPos = FVector2D(NormalizedX * (float)ViewRect.Width(), NormalizedY * (float)ViewRect.Height());
		return true;
	}
	return false;
}

/**
 * Approximates hardware mip-level selection with default anisotropic filtering.
 * TexelWS is the world-space position of the texel being shaded; TexelOff{X,Y}WS are
 * positions one source-texel away along each axis. Returns false if any of the three
 * points is behind the camera.
 */
inline bool CalculateMipLevelAniso(const FImgMediaViewInfo& ViewInfo, const FVector& TexelWS,
	const FVector& TexelOffXWS, const FVector& TexelOffYWS, float& OutMipLevel)
{
	// Cache the CVar pointer once but read its value each call: r.MaxAnisotropy can change at
	// runtime (PIE/editor settings), and the CVar may not be registered the very first time
	// this runs (commandlet/early-load) - dereferencing a static value snapshot would null-deref.
	static IConsoleVariable* MaxAnisoCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.MaxAnisotropy"));
	const int32 MaxAniso = MaxAnisoCVar ? FMath::Clamp(MaxAnisoCVar->GetInt(), 1, 16) : 1;
	const float MaxAnisoLog2 = FMath::Log2((float)MaxAniso);

	FVector2D TexelScreenSpace[3];
	bool bValid = true;
	bValid &= ProjectWorldToScreenFast(TexelWS, ViewInfo.ViewportRect, ViewInfo.ViewProjectionMatrix, TexelScreenSpace[0]);
	bValid &= ProjectWorldToScreenFast(TexelOffXWS, ViewInfo.ViewportRect, ViewInfo.ViewProjectionMatrix, TexelScreenSpace[1]);
	bValid &= ProjectWorldToScreenFast(TexelOffYWS, ViewInfo.ViewportRect, ViewInfo.ViewProjectionMatrix, TexelScreenSpace[2]);

	if (bValid)
	{
		// SafeDivide saturates to FLT_MAX at the projection singularity (sphere pole, adjacent texels
		// projecting to the same screen point). Without it, 1/0 = +inf flows through Floor/Ceil/clamp
		// and the per-tile mip range collapses to [0, MaxLevel], marking every mip visible.
		const float Px = FMath::SafeDivide(1.0f, static_cast<float>(FVector2D::DistSquared(TexelScreenSpace[0], TexelScreenSpace[1])));
		const float Py = FMath::SafeDivide(1.0f, static_cast<float>(FVector2D::DistSquared(TexelScreenSpace[0], TexelScreenSpace[2])));

		const float MinLevel = 0.5f * FMath::Log2(FMath::Min(Px, Py));
		const float MaxLevel = 0.5f * FMath::Log2(FMath::Max(Px, Py));

		const float AnisoBias = FMath::Min(MaxLevel - MinLevel, MaxAnisoLog2);
		OutMipLevel = MaxLevel - AnisoBias;
	}

	return bValid;
}

} // namespace UE::ImgMediaProviders
