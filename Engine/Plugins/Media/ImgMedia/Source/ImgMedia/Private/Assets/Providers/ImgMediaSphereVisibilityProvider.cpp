// Copyright Epic Games, Inc. All Rights Reserved.

#include "Assets/Providers/ImgMediaSphereVisibilityProvider.h"

#include "Assets/MediaSequenceInfo.h"
#include "Assets/Providers/ImgMediaProviderUtils.h"
#include "Async/Async.h"
#include "Components/MeshComponent.h"
#include "ConvexVolume.h"
#include "Engine/Engine.h"
#include "ImgMediaPrivate.h"
#include "ImgMediaSceneViewExtension.h"
#include "Math/UnrealMathUtility.h"
#include "UObject/GarbageCollection.h"

#if WITH_EDITOR
#include "Editor.h"
#include "DrawDebugHelpers.h"
#endif

namespace ImgMediaSphereProvider
{
	// Sphere-specific UV/position helpers. The shared CVar accessors and CalculateMipLevelAniso
	// live in ImgMediaProviderUtils.h.
	FORCEINLINE FVector TransformSphericalUVsToLocationWS(const FVector2f& MeshRange,
		const FTransform& MeshTransform, FVector2f UV, float SphereRadius)
	{
		// Scale UVs by spherical mesh range
		UV.X *= MeshRange.X / 360.0f;
		UV.Y = (UV.Y - 0.5f) * (MeshRange.Y / 180.0f) + 0.5f;
		// Convert from latlong UV to spherical coordinates
		FVector2d TileCornerSpherical = FVector2d(UE_PI * UV.Y, UE_TWO_PI * UV.X);
		FVector CornersWS = TileCornerSpherical.SphericalToUnitCartesian() * SphereRadius;
		return MeshTransform.TransformPosition(CornersWS);
	}

	FORCEINLINE FVector2f TransformDirectionWSToSphericalUVs(const FVector2f& MeshRange,
		const FTransform& MeshTransform, const FVector& InDirection)
	{
		// Convert direction to spherical angular coordinates.
		FVector SphereViewPoint = MeshTransform.InverseTransformVectorNoScale(InDirection);
		SphereViewPoint.Normalize();
		FVector2d Spherical = SphereViewPoint.UnitCartesianToSpherical();
		Spherical.Y = FMath::Fmod(Spherical.Y + UE_TWO_PI, UE_TWO_PI);
		// Convert spherical to 0-1 UV range.
		FVector2f UV = FVector2f(Spherical.Y / UE_TWO_PI, Spherical.X / UE_PI);
		// Scale UVs by spherical mesh range. UV may fall outside [0,1] for partial-range
		// meshes when the direction lies outside the mesh's coverage.
		UV.X /= MeshRange.X / 360.0f;
		UV.Y = (UV.Y - 0.5f) / (MeshRange.Y / 180.0f) + 0.5f;
		return UV;
	}
} // namespace ImgMediaSphereProvider


FImgMediaSphereVisibilityProvider::FImgMediaSphereVisibilityProvider(
	const FImgMediaSphereVisibilityProviderParams& InParams)
	: Params(InParams)
{
}

bool FImgMediaSphereVisibilityProvider::IsAlive() const
{
	return Params.MeshComponent.IsValid();
}

void FImgMediaSphereVisibilityProvider::GatherVisibleTiles(
	const FMediaSequenceInfo& InSequenceInfo,
	FMediaTileVisibilityRequest& OutRequest) const
{
	using namespace ImgMediaSphereProvider;
	using namespace UE::ImgMediaProviders;

	// Pin the mesh under FGCScopeGuard so GC can't reclaim it between the IsValid/ShouldRender
	// check and the property reads. We snapshot every UObject-derived value we need, then drop
	// the guard for the rest of GatherVisibleTiles - the math below works on plain values, no
	// further UObject access. Holding the guard across the per-tile sphere walk would stall GC
	// unnecessarily.
	FTransform MeshTransform;
	FPrimitiveComponentId MeshPrimitiveSceneId;
	{
		FGCScopeGuard GCGuard;
		UMeshComponent* Mesh = Params.MeshComponent.Get();
		if (Mesh == nullptr || !Mesh->ShouldRender())
		{
			return;
		}
		MeshTransform = Mesh->GetComponentTransform();
		MeshPrimitiveSceneId = Mesh->GetPrimitiveSceneId();
	}

	TSharedPtr<FImgMediaSceneViewExtension, ESPMode::ThreadSafe> SVE = Params.SceneViewExtension.Pin();
	if (!SVE.IsValid())
	{
		return;
	}

	// Select the appropriate set(s) of cached view infos based on the target resolution mask.
	TArray<FImgMediaViewInfo> ViewInfos = SVE->GetViewInfos(Params.TargetViewResolutionMask);

	if (ViewInfos.IsEmpty())
	{
		return;
	}

	// PixelDimX/PixelDimY below divide by the sequence pixel dimensions.
	if (InSequenceInfo.Dim.X <= 0 || InSequenceInfo.Dim.Y <= 0)
	{
		return;
	}

#if WITH_EDITOR
	const bool bDrawDebugSpheres = GetDebugDrawEnabled();

	// Batched into one TaskGraph dispatch at end of function; per-sphere Async measured
	// ~5x cost on dense visibility frames in the plane provider, same fix applies here.
	struct FPendingDebugSphere
	{
		FVector Center;
		float Radius;
	};
	TArray<FPendingDebugSphere> PendingDebugSpheres;
#endif
	const FVector2f MeshRange = FVector2f(Params.MeshRange);
	const float MipMapBias = Params.MipMapLODBias;
	const float MipMapLevelPadding = GetMipLevelPadding();
	const int32 MaxLevel = InSequenceInfo.NumMipLevels - 1;

	const FIntPoint& SequenceTileNum = InSequenceInfo.TilingDescription.TileNum;
	const FVector2f SequencePartialTileNum = InSequenceInfo.GetPartialTileNum();

	const float PixelDimX = 1.0f / InSequenceInfo.Dim.X;
	const float PixelDimY = 1.0f / InSequenceInfo.Dim.Y;

	const FVector ApproxTileSizeWS = MeshTransform.GetScale3D() * (UE_TWO_PI * DefaultSphereRadius)
		/ FMath::Max(SequencePartialTileNum.X, SequencePartialTileNum.Y);
	const int32 TotalNumTiles = FMath::CeilToInt32(SequencePartialTileNum.X * SequencePartialTileNum.Y);

	const float ApproxTileRadiusInWS = 0.5f * UE_SQRT_2 * ApproxTileSizeWS.GetAbsMax();

	// Adaptive pole upscaling state. The user-supplied upscale level is capped at the tile-pyramid
	// root so the upscale fallback is never sub-tile-sized. Asset mip chains run to 1x1, so without
	// this cap the UI default (MipLevelToUpscale=16) would clamp to NumMips-1 and produce a single
	// texel stretched over the entire sphere as the fallback.
	const int32 TilePyramidRoot = ComputeQuadtreeMaxMipLevel(
		InSequenceInfo.NumMipLevels, SequenceTileNum, InSequenceInfo.IsTiled());
	const int32 MipLevelToUpscaleExcludingPoles = (Params.MipLevelToUpscale >= 0)
		? FMath::Min(Params.MipLevelToUpscale, TilePyramidRoot)
		: Params.MipLevelToUpscale;
	const bool bAdaptivePoleMipUpscaling = Params.bAdaptivePoleMipUpscaling;
	int32 RuntimeMipLevelToUpscale = MipLevelToUpscaleExcludingPoles;

	for (const FImgMediaViewInfo& ViewInfo : ViewInfos)
	{
		if (IsPrimitiveComponentHidden(MeshPrimitiveSceneId, ViewInfo))
		{
			continue;
		}

		// Analytical derivation of visible tiles from the view frustum.
		FConvexVolume ViewFrustum;
		GetViewFrustumBounds(ViewFrustum, ViewInfo.OverscanViewProjectionMatrix, false, false);

		// Approximated UV coordinate for a camera centered inside the sphere.
		FVector2f ViewUV = TransformDirectionWSToSphericalUVs(MeshRange, MeshTransform, ViewInfo.ViewDirection);

		// 20 Degrees equates to total 11.1% of a sphere for top and bottom poles.
		const float PoleEdgeDegrees = 20.f;
		// Angle from the pole at which upscaling is enabled automatically.
		const float UpscalingEnabledEdge = 10.f;

		int32 NumOfTilesToLoadAtThePole = 0, PoleMipBias = -1;
		const float TileReductionFactor = 0.5f;
		if (bAdaptivePoleMipUpscaling)
		{
			NumOfTilesToLoadAtThePole = FMath::CeilToInt32(SequencePartialTileNum.X *
				((float)SequencePartialTileNum.Y * PoleEdgeDegrees / 180.f)) * TileReductionFactor;
			PoleMipBias = (NumOfTilesToLoadAtThePole > 0) ?
				FMath::CeilToInt32(FMath::LogX(4., TotalNumTiles / NumOfTilesToLoadAtThePole)) : 0;
		}

		const float TipUpscalingEdgePercent = (1.f - PoleEdgeDegrees / 90.f);
		const float TipInTheViewThresholdPercent = (1.f - UpscalingEnabledEdge / 90.f);
		bool bPoleTipIsInView = false;

		auto ProcessTileRow = [&](int32 TileY)
		{
			float TileVMin = 0, TileVMax = 0;
			if (bAdaptivePoleMipUpscaling)
			{
				TileVMin = ((((float)TileY) / SequencePartialTileNum.Y) - 0.5) * 2.;
				TileVMax = ((((float)TileY + 1.) / SequencePartialTileNum.Y) - 0.5) * 2.;
			}

			for (int32 TileX = 0; TileX < SequenceTileNum.X; ++TileX)
			{
				const FVector2f TileMinCornerUV = FVector2f((float)TileX, (float)TileY) / SequencePartialTileNum;
				const FVector2f TileMaxCornerUV = FVector2f(TileX + 1.0f, TileY + 1.0f) / SequencePartialTileNum;
				FVector2f TileUV = 0.5f * (TileMinCornerUV + TileMaxCornerUV);
				float CollisionSphereRadius = ApproxTileRadiusInWS;

				// If the view uv is inside the tile, we use its location directly. Helpful for sequences with no tiles.
				if (ViewUV.ComponentwiseAllGreaterOrEqual(TileMinCornerUV) && ViewUV.ComponentwiseAllLessThan(TileMaxCornerUV))
				{
					TileUV = ViewUV;
					CollisionSphereRadius *= 0.5f;
				}

				const FVector TileLocationWS = TransformSphericalUVsToLocationWS(MeshRange, MeshTransform, TileUV, DefaultSphereRadius);

				if (ViewFrustum.IntersectSphere(TileLocationWS, CollisionSphereRadius))
				{
					if (bAdaptivePoleMipUpscaling && (TileVMin < -TipInTheViewThresholdPercent || TileVMax > TipInTheViewThresholdPercent))
					{
						bPoleTipIsInView = true;
						RuntimeMipLevelToUpscale = FMath::Max(PoleMipBias, MipLevelToUpscaleExcludingPoles);
					}

					float CalculatedLevel;
					FIntVector2 MipLevelRange{0, 0};

					const FVector TexelOffXWS = TransformSphericalUVsToLocationWS(MeshRange, MeshTransform, TileUV + FVector2f(PixelDimX, 0), DefaultSphereRadius);
					const FVector TexelOffYWS = TransformSphericalUVsToLocationWS(MeshRange, MeshTransform, TileUV + FVector2f(0, PixelDimY), DefaultSphereRadius);

					if (CalculateMipLevelAniso(ViewInfo, TileLocationWS, TexelOffXWS, TexelOffYWS, CalculatedLevel))
					{
						CalculatedLevel += MipMapBias + ViewInfo.MaterialTextureMipBias;

						MipLevelRange[0] = FMath::FloorToInt32(CalculatedLevel - MipMapLevelPadding);
						MipLevelRange[1] = FMath::CeilToInt32(CalculatedLevel + MipMapLevelPadding);

						const bool bIsTileAtPole = (TileY == 0 || TileY == SequenceTileNum.Y - 1);
						/*
						 * Per-fragment mip selection diverges from per-tile estimation most strongly at the
						 * poles due to latlong projection. As a mitigation we artifically widen their range,
						 * unless adaptive pole upscaling is on (which addresses the same problem differently).
						 */
						if (!bAdaptivePoleMipUpscaling && bIsTileAtPole)
						{
							MipLevelRange[0]--;
							MipLevelRange[1]++;
						}

						MipLevelRange[0] = FMath::Clamp(MipLevelRange[0], 0, MaxLevel);
						MipLevelRange[1] = FMath::Clamp(MipLevelRange[1], 0, MaxLevel);

						for (int32 Level = MipLevelRange[0]; Level <= MipLevelRange[1]; ++Level)
						{
							if (bAdaptivePoleMipUpscaling && Level < PoleMipBias)
							{
								if (bPoleTipIsInView && (TileVMin < -TipUpscalingEdgePercent || TileVMax > TipUpscalingEdgePercent))
								{
									continue;
								}
							}

							if (!OutRequest.VisibleTiles.Contains(Level))
							{
								OutRequest.VisibleTiles.Emplace(Level,
									FMediaTileSelection::CreateForTargetMipLevel(InSequenceInfo.Dim, InSequenceInfo.TilingDescription.TileSize, Level, false));
							}

							const int MipLevelDiv = 1 << Level;

							OutRequest.VisibleTiles[Level].SetVisible(TileX / MipLevelDiv, TileY / MipLevelDiv);
						}
					}
#if WITH_EDITOR
					if (bDrawDebugSpheres)
					{
						const float DebugSphereRadius = FMath::Max(CollisionSphereRadius, MeshTransform.GetMaximumAxisScale());
						PendingDebugSpheres.Add({TileLocationWS, DebugSphereRadius});
					}
#endif // WITH_EDITOR
				}
			}
		};

		const int32 MiddleRowIndex = FMath::FloorToInt32(((float)SequenceTileNum.Y) / 2.);

		for (int32 TileY = 0; TileY < MiddleRowIndex; ++TileY)
		{
			ProcessTileRow(TileY);
		}
		for (int32 TileY = SequenceTileNum.Y - 1; TileY >= MiddleRowIndex; TileY--)
		{
			ProcessTileRow(TileY);
		}
	}

	if (RuntimeMipLevelToUpscale >= 0)
	{
		OutRequest.MipLevelToUpscale = RuntimeMipLevelToUpscale;
	}

#if WITH_EDITOR
	if (!PendingDebugSpheres.IsEmpty())
	{
		Async(EAsyncExecution::TaskGraphMainThread, [Spheres = MoveTemp(PendingDebugSpheres)]()
		{
			if (const UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : GWorld)
			{
				for (const FPendingDebugSphere& Sphere : Spheres)
				{
					DrawDebugSphere(World, Sphere.Center, Sphere.Radius, 8, FColor::Red, false, 0.05f);
				}
			}
		});
	}
#endif
}

namespace UE::ImgMediaSphereVisibility::Tests
{
	FVector TransformSphericalUVsToLocationWS(const FVector2f& InMeshRange,
		const FTransform& InMeshTransform, FVector2f InUV, float InSphereRadius)
	{
		return ImgMediaSphereProvider::TransformSphericalUVsToLocationWS(InMeshRange, InMeshTransform, InUV, InSphereRadius);
	}
	
	FVector2f TransformDirectionWSToSphericalUVs(const FVector2f& InMeshRange,
		const FTransform& InMeshTransform, const FVector& InDirection)
	{
		return ImgMediaSphereProvider::TransformDirectionWSToSphericalUVs(InMeshRange, InMeshTransform, InDirection);
	}
}