// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Assets/MediaTileVisibility.h"
#include "Assets/Providers/ImgMediaProviderUtils.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"

/** Construction parameters for the sphere visibility provider; adds sphere-only fields on top of the base. */
struct FImgMediaSphereVisibilityProviderParams : public FImgMediaVisibilityProviderParams
{
	/** Whether to dynamically raise the upscale level near sphere poles to reduce load. */
	bool bAdaptivePoleMipUpscaling = false;

	/** Spherical mesh range (X = horizontal degrees, Y = vertical degrees). */
	FVector2D MeshRange = FVector2D(360.0, 180.0);
};

/**
 * Visibility provider for media displayed on a latlong sphere.
 * Performs an analytical UV walk over the sphere's tile grid and emits tile
 * selections per visible mip level. Optionally raises the upscale mip level
 * near the poles when bAdaptivePoleMipUpscaling is set.
 */
class FImgMediaSphereVisibilityProvider final : public FMediaTileVisibilityProvider
{
public:
	explicit FImgMediaSphereVisibilityProvider(const FImgMediaSphereVisibilityProviderParams& InParams);

	//~ Begin IMediaTileVisibilityProvider
	virtual void GatherVisibleTiles(const FMediaSequenceInfo& InSequenceInfo,
		FMediaTileVisibilityRequest& OutRequest) const override;
	virtual bool IsAlive() const override;
	//~ End IMediaTileVisibilityProvider

private:
	static constexpr float DefaultSphereRadius = 50.0f; // matches FMediaPlateCustomizationMesh::GenerateSphereMesh

	FImgMediaSphereVisibilityProviderParams Params;
};

// Exposed for tests
namespace UE::ImgMediaSphereVisibility::Tests
{
	/** Maps a UV in [0,1]^2 (mesh space) to its world-space position on a partial-range latlong sphere. */
	FVector TransformSphericalUVsToLocationWS(const FVector2f& InMeshRange,
		const FTransform& InMeshTransform, FVector2f InUV, float InSphereRadius);

	/**
	 * Inverse of TransformSphericalUVsToLocationWS. Result UV may fall outside [0,1] when the
	 * direction lies outside a partial mesh's coverage - that is the correct semantic for
	 * "camera looking past the partial sphere".
	 */
	FVector2f TransformDirectionWSToSphericalUVs(const FVector2f& InMeshRange,
		const FTransform& InMeshTransform, const FVector& InDirection);
}