// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImgMediaGlobalCache.h"
#include "Readers/IImgMediaReader.h"
#include "ImgMediaPrivate.h"
#include "ImgMediaSettings.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR

#include "Async/Async.h"
#include "IDirectoryWatcher.h"
#include "DirectoryWatcherModule.h"
#include "Misc/Paths.h"

#endif

FImgMediaGlobalCache::FImgMediaGlobalCache()
	: LeastRecent(nullptr)
	, MostRecent(nullptr)
	, CurrentSize(0)
	, MaxSize(0)
{
}

FImgMediaGlobalCache::~FImgMediaGlobalCache()
{
	Shutdown();
}

void FImgMediaGlobalCache::Initialize()
{
	auto Settings = GetDefault<UImgMediaSettings>();
#if WITH_EDITOR
	UpdateSettingsDelegateHandle = UImgMediaSettings::OnSettingsChanged().AddThreadSafeSP(this, &FImgMediaGlobalCache::UpdateSettings);
#endif
	UpdateSettings(Settings);
}

void FImgMediaGlobalCache::Shutdown()
{
	FScopeLock Lock(&CriticalSection);
	Empty();

#if WITH_EDITOR
	// Clean up watchers.
	static const FName NAME_DirectoryWatcher = "DirectoryWatcher";
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(NAME_DirectoryWatcher);
	IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get();
	if (DirectoryWatcher != nullptr)
	{
		for (const TPair<FString, FDelegateHandle>& Elem : MapSequenceToWatcher)
		{
			DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(Elem.Key, Elem.Value);
		}
	}
	MapSequenceToWatcher.Empty();

	UImgMediaSettings::OnSettingsChanged().Remove(UpdateSettingsDelegateHandle);
#endif // WITH_EDITOR
}

void FImgMediaGlobalCache::AddFrame(const FString& FileName, const FName& Sequence, int32 Index, const TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>& Frame, bool bAdjustSizeForMipMaps)
{
	FScopeLock Lock(&CriticalSection);

	// Make sure we have enough space in the cache to add this new frame.
	SIZE_T FrameSize = Frame->GetUncompressedSize();
	if (bAdjustSizeForMipMaps)
	{
		FrameSize = (FrameSize * 4) / 3;
	}
	if (FrameSize <= MaxSize)
	{
#if WITH_EDITOR
		// Set up directory watcher so we know when our source files change.
		FString FilePath = FPaths::GetPath(Sequence.ToString());
		if (MapSequenceToWatcher.Contains(FilePath) == false)
		{
			// Directory watcher needs to be called from the main thread.
			AsyncTask(ENamedThreads::GameThread, [this, FilePath]()
			{
				FScopeLock Lock(&CriticalSection);

				// Make sure we haven't been added already.
				if (MapSequenceToWatcher.Contains(FilePath) == false)
				{
					// Set up watcher.
					static const FName NAME_DirectoryWatcher = "DirectoryWatcher";
					FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(NAME_DirectoryWatcher);
					IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get();
					if (DirectoryWatcher != nullptr)
					{
						FDelegateHandle Handle;
						IDirectoryWatcher::FDirectoryChanged Callback = IDirectoryWatcher::FDirectoryChanged::CreateRaw(this, &FImgMediaGlobalCache::OnDirectoryChanged);

						DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(
							FilePath,
							Callback,
							Handle,
							IDirectoryWatcher::WatchOptions::IncludeDirectoryChanges
						);
						MapSequenceToWatcher.Emplace(FilePath, Handle);
					}
				}
			});
		}
#endif // WITH_EDITOR

		// Empty cache until we have enough space.
		EnforceMaxSize(FrameSize);

		// Create new entry.
		FImgMediaGlobalCacheEntry* NewEntry = new FImgMediaGlobalCacheEntry(FileName, FrameSize, Index, Frame);
		MapFrameToEntry.Emplace(TPair<FName, int32>(Sequence, Index), NewEntry);
#if WITH_EDITOR
		MapFileToIndex.Emplace(FileName, Index);
#endif

		MarkAsRecent(Sequence, *NewEntry);

		CurrentSize += FrameSize;
	}
	else
	{
		UE_LOG(LogImgMedia, Warning, TEXT("Global cache size %" SIZE_T_FMT " is smaller than frame size %" SIZE_T_FMT "."), MaxSize, FrameSize);
	}
}

bool FImgMediaGlobalCache::Contains(const FName& Sequence, int32 Index)
{
	FScopeLock Lock(&CriticalSection);

	return MapFrameToEntry.Contains(TPair<FName, int32>(Sequence, Index));
}

TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>* FImgMediaGlobalCache::FindAndTouch(const FName& Sequence, int32 Index)
{
	FScopeLock Lock(&CriticalSection);

	TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>* Frame = nullptr;

	FImgMediaGlobalCacheEntry** Entry = MapFrameToEntry.Find(TPair<FName, int32>(Sequence, Index));

	if ((Entry != nullptr) && (*Entry != nullptr))
	{
		Frame = &((*Entry)->Frame);

		// Mark this as the most recent.
		Unlink(Sequence, **Entry);
		MarkAsRecent(Sequence, **Entry);
	}

	return Frame;
}

void FImgMediaGlobalCache::GetIndices(const FName& Sequence, TArray<int32>& OutIndices) const
{
	FScopeLock Lock(&CriticalSection);

	// Get most recent entry in this sequence.
	FImgMediaGlobalCacheEntry* const* CurrentPtr = MapSequenceToMostRecentEntry.Find(Sequence);
	const FImgMediaGlobalCacheEntry* Current = CurrentPtr != nullptr ? *CurrentPtr : nullptr;

	// Loop over all entries in the sequence.
	while (Current != nullptr)
	{
		OutIndices.Add(Current->Index);
		Current = Current->LessRecentSequence;
	}
}

void FImgMediaGlobalCache::EmptyCache()
{
	FScopeLock Lock(&CriticalSection);
	Empty();
}

void FImgMediaGlobalCache::EnforceMaxSize(SIZE_T Extra)
{
	while (CurrentSize + Extra > MaxSize)
	{
		FName* RemoveSequencePtr = MapLeastRecentToSequence.Find(LeastRecent);
		FName RemoveSequence = RemoveSequencePtr != nullptr ? *RemoveSequencePtr : FName();
		Remove(RemoveSequence, *LeastRecent);
	}
}

void FImgMediaGlobalCache::Remove(const FName& Sequence, FImgMediaGlobalCacheEntry& Entry)
{
	// Remove from cache.
	Unlink(Sequence, Entry);

	// Update current cache size.
	SIZE_T FrameSize = Entry.FrameSize;
	CurrentSize -= FrameSize;

	// Delete entry.
	MapFrameToEntry.Remove(TPair<FName, int32>(Sequence, Entry.Index));
#if WITH_EDITOR
	MapFileToIndex.Remove(Entry.FileName);
#endif
	delete &Entry;
}

void FImgMediaGlobalCache::MarkAsRecent(const FName& Sequence, FImgMediaGlobalCacheEntry& Entry)
{
	// Mark most recent.
	Entry.LessRecent = MostRecent;
	if (MostRecent != nullptr)
	{
		MostRecent->MoreRecent = &Entry;
	}
	MostRecent = &Entry;

	// Mark most recent in sequence.
	FImgMediaGlobalCacheEntry** SequenceMostRecentPtr = MapSequenceToMostRecentEntry.Find(Sequence);
	FImgMediaGlobalCacheEntry* SequenceMostRecent = (SequenceMostRecentPtr != nullptr) ? (*SequenceMostRecentPtr) : nullptr;
	Entry.LessRecentSequence = SequenceMostRecent;
	if (SequenceMostRecent != nullptr)
	{
		SequenceMostRecent->MoreRecentSequence = &Entry;
	}
	else
	{
		// If we did not have a most recent one, then this is the first in the sequence.
		MapLeastRecentToSequence.Emplace(&Entry, Sequence);
	}
	MapSequenceToMostRecentEntry.Emplace(Sequence, &Entry);

	// If LeastRecent is null, then set it now.
	if (LeastRecent == nullptr)
	{
		LeastRecent = &Entry;
	}
}

void FImgMediaGlobalCache::Unlink(const FName& Sequence, FImgMediaGlobalCacheEntry& Entry)
{
	// Remove from link.
	if (Entry.LessRecent != nullptr)
	{
		Entry.LessRecent->MoreRecent = Entry.MoreRecent;
	}
	else if (LeastRecent == &Entry)
	{
		LeastRecent = Entry.MoreRecent;
	}

	if (Entry.MoreRecent != nullptr)
	{
		Entry.MoreRecent->LessRecent = Entry.LessRecent;
	}
	else if (MostRecent == &Entry)
	{
		MostRecent = Entry.LessRecent;
	}

	Entry.LessRecent = nullptr;
	Entry.MoreRecent = nullptr;

	// Remove from sequence link.
	if (Entry.LessRecentSequence != nullptr)
	{
		Entry.LessRecentSequence->MoreRecentSequence = Entry.MoreRecentSequence;
	}
	else
	{
		MapLeastRecentToSequence.Remove(&Entry);
		if (Entry.MoreRecentSequence != nullptr)
		{
			MapLeastRecentToSequence.Emplace(Entry.MoreRecentSequence, Sequence);
		}
	}

	if (Entry.MoreRecentSequence != nullptr)
	{
		Entry.MoreRecentSequence->LessRecentSequence = Entry.LessRecentSequence;
	}
	else
	{
		// Update most recent in sequence.
		FImgMediaGlobalCacheEntry** MostRecentSequence = MapSequenceToMostRecentEntry.Find(Sequence);
		if (MostRecentSequence != nullptr)
		{
			if (*MostRecentSequence == &Entry)
			{
				MapSequenceToMostRecentEntry.Emplace(Sequence, Entry.LessRecentSequence);
			}
		}
	}

	Entry.LessRecent = nullptr;
	Entry.LessRecentSequence = nullptr;
	Entry.MoreRecent = nullptr;
	Entry.MoreRecentSequence = nullptr;
}

void FImgMediaGlobalCache::Empty()
{
	while (LeastRecent != nullptr)
	{
		FImgMediaGlobalCacheEntry *Entry = LeastRecent;
		LeastRecent = Entry->MoreRecent;

		delete Entry;
	}

	MostRecent = nullptr;
	CurrentSize = 0;

	MapSequenceToMostRecentEntry.Empty();
	MapLeastRecentToSequence.Empty();
	MapFrameToEntry.Empty();
}

void FImgMediaGlobalCache::UpdateSettings(const UImgMediaSettings* Settings)
{
	MaxSize = Settings->GlobalCacheSizeGB * 1024 * 1024 * 1024;
	EnforceMaxSize(0);
}

#if WITH_EDITOR

void FImgMediaGlobalCache::OnDirectoryChanged(const TArray<FFileChangeData>& InFileChanges)
{
	FScopeLock Lock(&CriticalSection);

	// Loop over all file changes.
	for (FFileChangeData FileChangeData : InFileChanges)
	{
		// Do we know about this file?
		int32* Index = MapFileToIndex.Find(FileChangeData.Filename);
		if (Index != nullptr)
		{
			// Remove this entry if we have it.
			FString SequenceString = FPaths::GetPath(FileChangeData.Filename).AppendChar(TEXT('/'));
			FName SequenceName = FName(*SequenceString);
			FImgMediaGlobalCacheEntry** CacheEntry = MapFrameToEntry.Find(TPair<FName, int32>(SequenceName, *Index));
			if ((CacheEntry != nullptr) && (*CacheEntry != nullptr))
			{
				Remove(SequenceName, **CacheEntry);
			}
		}
	}
}

#endif // WITH_EDITOR
