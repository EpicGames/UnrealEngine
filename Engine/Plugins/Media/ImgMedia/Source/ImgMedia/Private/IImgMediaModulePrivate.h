// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IImgMediaModule.h"

/**
 * Extended Private Interface for the ImgMedia module.
 */
class IImgMediaModulePrivate
	: public IImgMediaModule
{
public:
	/**
	 * Returns a pointer to the module instance if loaded, null otherwise.
	 */
	static inline IImgMediaModulePrivate* Get()
	{
		return FModuleManager::GetModulePtr<IImgMediaModulePrivate>("ImgMedia");
	}

	/**
	 * Returns existing instance of the module, loads it if it was not already loaded.
	 * Beware of calling this during the shutdown phase, though. The module might have been unloaded already.
	 */
	static IImgMediaModulePrivate& GetOrLoad()
	{
		return FModuleManager::LoadModuleChecked<IImgMediaModulePrivate>("ImgMedia");
	}
};