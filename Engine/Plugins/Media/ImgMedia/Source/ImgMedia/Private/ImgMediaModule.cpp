// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileMediaSource.h"
#include "IImgMediaModule.h"
#include "IImgMediaModulePrivate.h"
#include "IMediaClock.h"
#include "IMediaModule.h"
#include "ImgMediaGlobalCache.h"
#include "ImgMediaPrivate.h"
#include "ImgMediaSceneViewExtension.h"
#include "ImgMediaSource.h"
#include "ITmvMediaModule.h"
#include "Decoder/ITmvMediaDecoderFactory.h"
#include "Decoder/ITmvMediaDemuxerFactory.h"
#include "Misc/QueuedThreadPool.h"
#include "Player/ImgMediaPlayer.h"
#include "Scheduler/ImgMediaScheduler.h"

CSV_DEFINE_CATEGORY_MODULE(IMGMEDIA_API, ImgMedia, false);
DEFINE_LOG_CATEGORY(LogImgMedia);

FLazyName IImgMediaModule::CustomFormatAttributeName(TEXT("EpicGamesCustomFormat"));
FLazyName IImgMediaModule::CustomFormatTileWidthAttributeName(TEXT("EpicGamesTileWidth"));
FLazyName IImgMediaModule::CustomFormatTileHeightAttributeName(TEXT("EpicGamesTileHeight"));
FLazyName IImgMediaModule::CustomFormatTileBorderAttributeName(TEXT("EpicGamesTileBorder"));

TSharedPtr<FImgMediaGlobalCache, ESPMode::ThreadSafe> IImgMediaModule::GlobalCache;

#if USE_IMGMEDIA_DEALLOC_POOL
struct FImgMediaThreadPool
{
public:

	FImgMediaThreadPool() :
		Pool(nullptr),
		bHasInit(false)
	{
	}

	~FImgMediaThreadPool()
	{
		Reset();
	}

	void Reset()
	{
		FScopeLock Lock(&CriticalSection);
		if (Pool != nullptr)
		{
			Pool->Destroy();
			Pool = nullptr;
		}

		bHasInit = false;
	}

	FQueuedThreadPool* GetThreadPool()
	{
		FScopeLock Lock(&CriticalSection);
		if (bHasInit)
		{
			return Pool;
		}

		// initialize worker thread pools
		if (FPlatformProcess::SupportsMultithreading())
		{
			// initialize dealloc thread pool
			const int32 ThreadPoolSize = 1;
			const uint32 StackSize = 128 * 1024;

			Pool = FQueuedThreadPool::Allocate();
			verify(Pool->Create(ThreadPoolSize, StackSize, TPri_Normal));
		}

		bHasInit = true;

		return Pool;
	}

private:
	FCriticalSection CriticalSection;
	FQueuedThreadPool* Pool;
	bool bHasInit;
};

FImgMediaThreadPool ImgMediaThreadPool;

FQueuedThreadPool* GetImgMediaThreadPoolSlow()
{
	return ImgMediaThreadPool.GetThreadPool();
}
#endif // USE_IMGMEDIA_DEALLOC_POOL


/**
 * Implements the AVFMedia module.
 */
class FImgMediaModule
	: public IImgMediaModulePrivate
{
public:

	/** Default constructor. */
	FImgMediaModule() { }

public:

	//~ IImgMediaModule interface

	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
		if (!Scheduler.IsValid())
		{
			InitScheduler();
		}
		if (!GlobalCache.IsValid())
		{
			InitGlobalCache();
		}

		TSharedPtr<FImgMediaPlayer, ESPMode::ThreadSafe> Player = MakeShared<FImgMediaPlayer, ESPMode::ThreadSafe>(EventSink, Scheduler.ToSharedRef(), GlobalCache.ToSharedRef());
		OnImgMediaPlayerCreated.Broadcast(Player);

		return Player;
	}

	const TSharedPtr<FImgMediaSceneViewExtension, ESPMode::ThreadSafe>& GetSceneViewExtension() const override
	{
		return SceneViewExtension;
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		// Register media source spawners.
		FMediaSourceSpawnDelegate SpawnDelegate =
			FMediaSourceSpawnDelegate::CreateRaw(this, &FImgMediaModule::SpawnMediaSourceForString);

		for (const FString& Ext : FileExtensions)
		{
			UMediaSource::RegisterSpawnFromFileExtension(Ext, SpawnDelegate);
		}

		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FImgMediaModule::OnPostEngineInit);
		FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddRaw(this, &FImgMediaModule::OnAllModuleLoadingPhasesComplete);
		FCoreDelegates::OnEnginePreExit.AddRaw(this, &FImgMediaModule::OnEnginePreExit);

		ConsoleCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("ImgMedia.EmptyGlobalCache"),
			TEXT("Empty the ImgMedia Module's global cache."),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FImgMediaModule::EmptyGlobalCacheCommand),
			ECVF_Default
			));
	}

	virtual void ShutdownModule() override
	{
		for (IConsoleObject* ConsoleCmd : ConsoleCmds)
		{
			IConsoleManager::Get().UnregisterConsoleObject(ConsoleCmd);
		}
		ConsoleCmds.Empty();
		
		// Unregister media source spawners.
		for (const FString& Ext : FileExtensions)
		{
			UMediaSource::UnregisterSpawnFromFileExtension(Ext);
		}

		UnregisterTmvMediaFactories();

		Scheduler.Reset();
		GlobalCache.Reset();

		FCoreDelegates::OnEnginePreExit.RemoveAll(this);
		FCoreDelegates::OnAllModuleLoadingPhasesComplete.RemoveAll(this);
		FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

#if USE_IMGMEDIA_DEALLOC_POOL
		ImgMediaThreadPool.Reset();
#endif
	}

private:

	void OnPostEngineInit()
	{
		SceneViewExtension = FSceneViewExtensions::NewExtension<FImgMediaSceneViewExtension>();
	}

	void OnAllModuleLoadingPhasesComplete()
	{
		RegisterTmvMediaFactories();
	}

	/** Register media source spawners from TmvMedia decoder and demuxer factories. */
	void RegisterTmvMediaFactories()
	{
		const ITmvMediaModule* TmvMediaModule = ITmvMediaModule::Get();
		if (!TmvMediaModule)
		{
			return;
		}

		FMediaSourceSpawnDelegate SpawnDelegate = FMediaSourceSpawnDelegate::CreateRaw(this, &FImgMediaModule::SpawnMediaSourceForString);

		// Register all the tmv decoder factories for file sequence.
		{
			TArray<TWeakPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe>> DecoderFactories;
			TmvMediaModule->GetDecoderFactories(DecoderFactories);

			for (const TWeakPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe>& DecoderFactoryWeak : DecoderFactories)
			{
				if (const TSharedPtr<ITmvMediaDecoderFactory> DecoderFactory = DecoderFactoryWeak.Pin())
				{
					for (const FString& Extension : DecoderFactory->GetSupportedFileExtensions())
					{
						if (!TmvMediaFileExtensions.Contains(Extension))
						{
							UMediaSource::RegisterSpawnFromFileExtension(Extension, SpawnDelegate);
							TmvMediaFileExtensions.Add(Extension);
						}
					}
				}
			}
		}

		// Register all tmv demuxer factories for container.
		{
			TArray<TWeakPtr<ITmvMediaDemuxerFactory, ESPMode::ThreadSafe>> DemuxerFactories;
			TmvMediaModule->GetDemuxerFactories(DemuxerFactories);

			for (const TWeakPtr<ITmvMediaDemuxerFactory, ESPMode::ThreadSafe>& DemuxerFactoryWeak : DemuxerFactories)
			{
				if (const TSharedPtr<ITmvMediaDemuxerFactory> DemuxerFactory = DemuxerFactoryWeak.Pin())
				{
					for (const FString& Extension : DemuxerFactory->GetSupportedContainerFormats())
					{
						if (!TmvMediaFileExtensions.Contains(Extension))
						{
							UMediaSource::RegisterSpawnFromFileExtension(Extension, SpawnDelegate);
							TmvMediaFileExtensions.Add(Extension);
							TmvMediaContainerExtensions.Add(Extension);
						}
					}
				}
			}
		}
	}
	
	void UnregisterTmvMediaFactories()
	{
		for (const FString& Extension : TmvMediaFileExtensions)
		{
			UMediaSource::UnregisterSpawnFromFileExtension(Extension);
		}
	}

	void OnEnginePreExit()
	{
		SceneViewExtension.Reset();
	}

	void InitScheduler()
	{
		// initialize scheduler
		Scheduler = MakeShared<FImgMediaScheduler, ESPMode::ThreadSafe>();
		Scheduler->Initialize();

		IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->GetClock().AddSink(Scheduler.ToSharedRef());
		}
	}

	void InitGlobalCache()
	{
		// Initialize global cache.
		GlobalCache = MakeShared<FImgMediaGlobalCache, ESPMode::ThreadSafe>();
		GlobalCache->Initialize();
	}

	void EmptyGlobalCacheCommand(const TArray<FString>& InArgs)
	{
		if (GlobalCache)
		{
			GlobalCache->EmptyCache();
			UE_LOGF(LogImgMedia, Log, "GlobalCache Emptied.");
		}
		else
		{
			UE_LOGF(LogImgMedia, Warning, "Unable to empty GlobalCache. It is either deleted or not created yet.");
		}
	}

	/**
	 * Creates a media source for MediaPath.
	 *
	 * @param	MediaPath		File path to the media.
	 * @param	Outer			Outer to use for this object.
	 */
	UMediaSource* SpawnMediaSourceForString(const FString& MediaPath, UObject* Outer) const
	{
		// For supported container extensions, we will create a UFileMediaSource.
		if (TmvMediaContainerExtensions.Contains(FPaths::GetExtension(MediaPath)))
		{
			TObjectPtr<UFileMediaSource> FileMediaSource = NewObject<UFileMediaSource>(Outer, NAME_None, RF_Transactional);
			FileMediaSource->SetFilePath(MediaPath);
			return FileMediaSource;
		}

		// All other extensions are file sequence.
		TObjectPtr<UImgMediaSource> ImgMediaSource = NewObject<UImgMediaSource>(Outer, NAME_None, RF_Transactional);
		ImgMediaSource->SetSequencePath(MediaPath);
		return ImgMediaSource;
	}

	TSharedPtr<FImgMediaScheduler, ESPMode::ThreadSafe> Scheduler;

	/** Scene view extension used to track view/camera info. */
	TSharedPtr<FImgMediaSceneViewExtension, ESPMode::ThreadSafe> SceneViewExtension;

	/** List of file extensions that we support. */
	const TArray<FString> FileExtensions =
	{
		TEXT("bmp"),
		TEXT("exr"),
		TEXT("jpg"),
		TEXT("jpeg"),
		TEXT("png"),
		TEXT("dds")
	};
	
	/** Extra file extensions supported by Tmv Media Module factories. */
	TSet<FString> TmvMediaFileExtensions;

	/** File extensions supported by Tmv Container factories. */
	TSet<FString> TmvMediaContainerExtensions;

	/** Registered console commands. */
	TArray<IConsoleObject*> ConsoleCmds;
};


IMPLEMENT_MODULE(FImgMediaModule, ImgMedia);
