// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImgMediaSettings.h"
#include "Logging/LogMacros.h"
#include "Stats/Stats.h"
#include "UObject/NameTypes.h"

class FQueuedThreadPool;


/** The OpenEXR image reader is supported on macOS, Windows, and Linux only. */
#define IMGMEDIA_EXR_SUPPORTED_PLATFORM (PLATFORM_MAC || PLATFORM_WINDOWS || (PLATFORM_LINUX && PLATFORM_CPU_X86_FAMILY))

/** Whether to use a separate thread pool for image frame deallocations. */
#define USE_IMGMEDIA_DEALLOC_POOL UE_BUILD_DEBUG


/** Log category for this module. */
DECLARE_LOG_CATEGORY_EXTERN(LogImgMedia, Log, All);

/** Stat category for this module. */
DECLARE_STATS_GROUP(TEXT("ImgMediaPlugin"), STATGROUP_ImgMediaPlugin, STATCAT_Advanced);


#if USE_IMGMEDIA_DEALLOC_POOL
	/** Thread pool used for deleting image frame buffers. */
	extern FQueuedThreadPool* GetImgMediaThreadPoolSlow();
#endif

namespace ImgMedia
{
	/** Default frame rate for image sequences (24 fps). */
	static const FFrameRate DefaultFrameRate(24, 1);

	/** Name of the FillGapsInSequence media option. */
	static const FName FillGapsInSequenceOption("FillGapsInSequence");

	/** Name of the FramesRateOverrideDenonimator media option. */
	static const FName FrameRateOverrideDenonimatorOption("FrameRateOverrideDenonimator");

	/** Name of the FramesRateOverrideNumerator media option. */
	static const FName FrameRateOverrideNumeratorOption("FrameRateOverrideNumerator");

	/** Name of the ProxyOverride media option. */
	static const FName ProxyOverrideOption("ProxyOverride");

	/** Name of the LayerName media option. */
	static const FName LayerNameOption("LayerName");

	/** Name of the NumTilesX media option. */
	static const FName NumTilesXOption("ImgNumTilesX");

	/** Name of the NumTilesY media option. */
	static const FName NumTilesYOption("ImgNumTilesY");

	/** Name of the SourceColorSettings media option. */
	static const FName SourceColorSettingsOption("SourceColorSettings");
}
