// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImgMediaSceneViewExtension.h"
#include "DynamicResolutionState.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ImgMediaPrivate.h"
#include "SceneManagement.h"
#include "SceneView.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "ImgMediaSceneViewExtension"

static TAutoConsoleVariable<float> CVarImgMediaFieldOfViewMultiplier(
	TEXT("ImgMedia.FieldOfViewMultiplier"),
	1.0f,
	TEXT("Multiply the field of view for active cameras by this value, generally to increase the frustum overall sizes to mitigate missing tile artifacts.\n"),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarImgMediaProcessTilesInnerOnly(
	TEXT("ImgMedia.ICVFX.InnerOnlyTiles"),
	false,
	TEXT("This CVar will ignore tile calculation for all viewports except for Display Cluster inner viewports. User should enable upscaling on Media plate to display lower quality mips instead, otherwise ")
	TEXT("other viewports will only display tiles loaded specifically for inner viewport and nothing else. \n"),
#if WITH_EDITOR
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* CVar)
	{
		if (CVar->GetBool())
		{
			FNotificationInfo Info(LOCTEXT("EnableUpscalingNotification", "Tile calculation enabled for Display Cluster Inner Viewports exclusively.\nUse Mip Upscaling option on Media Plate to fill empty texture areas with lower quality data."));
			// Expire in 5 seconds.
			Info.ExpireDuration = 5.0f;
			FSlateNotificationManager::Get().AddNotification(Info);
		}
	}),
#endif
	ECVF_Default);

namespace
{
	/**
	 * True when CVarImgMediaProcessTilesInnerOnly is on and InViewFamily isn't a Display Cluster
	 * ICVFX in-camera viewport. Relies on DisplayClusterMediaHelpers::GenerateICVFXViewportName
	 * to have two strings embedded in ProfileDescription.
	 */
	bool IsViewFamilyFilteredOut(const FSceneViewFamily& InViewFamily)
	{
		return CVarImgMediaProcessTilesInnerOnly.GetValueOnGameThread()
			&& !(InViewFamily.ProfileDescription.Contains("_icvfx_") && InViewFamily.ProfileDescription.Contains("_incamera"));
	}
}

FImgMediaSceneViewExtension::FImgMediaSceneViewExtension(const FAutoRegister& AutoReg)
	: FSceneViewExtensionBase(AutoReg)
	, CachedViewInfos()
	, DisplayResolutionCachedViewInfos()
{
}

FImgMediaSceneViewExtension::~FImgMediaSceneViewExtension()
{
}

TArray<FImgMediaViewInfo> FImgMediaSceneViewExtension::GetViewInfos(EMediaTextureTargetViewResolution InTargetViewResolutionMask) const
{
	FScopeLock Lock(&ViewInfosMutex);
		
	TArray<FImgMediaViewInfo> ViewInfos;

	// When the display-resolution cache is empty, fall back to the render-resolution cache for
	// any caller asking for display - the common case where render and display match.
	const bool bDisplayCacheIsEmpty = DisplayResolutionCachedViewInfos.IsEmpty();
	if (EnumHasAllFlags(InTargetViewResolutionMask,
		EMediaTextureTargetViewResolution::RenderResolution) || bDisplayCacheIsEmpty)
	{
		ViewInfos.Append(CachedViewInfos);
	}
	if (EnumHasAllFlags(InTargetViewResolutionMask,
		EMediaTextureTargetViewResolution::DisplayResolution) && !bDisplayCacheIsEmpty)
	{
		ViewInfos.Append(DisplayResolutionCachedViewInfos);
	}

	return ViewInfos;
}

void FImgMediaSceneViewExtension::BeginRenderViewFamily(FSceneViewFamily& InViewFamily)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FImgMediaSceneViewExtension::BeginRenderViewFamily);

	{
		FScopeLock Lock(&ViewInfosMutex);

		// Reset only on a new frame, immediately before we repopulate. Multiple view families in the
		// same frame (e.g. split-screen, multi-viewport, nDisplay) must all accumulate, so subsequent
		// calls within the same GFrameNumber skip the reset and append to CachedViewInfos.
		const uint64 CurrentFrameNumber = GFrameNumber;
		if (CurrentFrameNumber != LastFrameNumber)
		{
			ResetViewInfoCache_NoLock();
			LastFrameNumber = CurrentFrameNumber;
		}

		for (const FSceneView* View : InViewFamily.Views)
		{
			if (View != nullptr)
			{
				CacheViewInfo_NoLock(InViewFamily, *View);
			}
		}
	}

	// Debug overlay: hoisted out of CacheViewInfo_NoLock so AddOnScreenDebugMessage doesn't
	// run while we hold ViewInfosMutex (it acquires GEngine's own internal lock).
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	static const IConsoleVariable* CVarMipMapDebug = IConsoleManager::Get().FindConsoleVariable(TEXT("ImgMedia.MipMapDebug"));

	if (!IsViewFamilyFilteredOut(InViewFamily) && GEngine != nullptr && CVarMipMapDebug != nullptr && CVarMipMapDebug->GetBool())
	{
		// Engine frame counter, so a glitch seen on screen can be matched to the
		// [N] bracket in log lines when correlating a recording against the log.
		GEngine->AddOnScreenDebugMessage(static_cast<int32>(0xF1A33044u), 0.25f, FColor::Yellow,
			*FString::Printf(TEXT("GFrameCounter: %llu"), static_cast<unsigned long long>(GFrameCounter)));

		const FString ViewName = InViewFamily.ProfileDescription.IsEmpty() ? TEXT("View") : InViewFamily.ProfileDescription;
		for (const FSceneView* View : InViewFamily.Views)
		{
			if (View == nullptr)
			{
				continue;
			}
			int32 Key = -1;
			float TimeToDisplay = 0.0f;
			if (View->State)
			{
				Key = View->State->GetViewKey();
				TimeToDisplay = 0.25f; // Reduce flicker for views that refresh at a slower rate than main viewport.
			}
			GEngine->AddOnScreenDebugMessage(Key, TimeToDisplay, FColor::Cyan,
				*FString::Printf(TEXT("%s location: [%s], direction: [%s]"),
					*ViewName,
					*View->ViewMatrices.GetViewOrigin().ToString(),
					*View->GetViewDirection().ToString()));
		}
	}
#endif
}

int32 FImgMediaSceneViewExtension::GetPriority() const
{
	// Lowest priority value to ensure all other extensions are executed before ours.
	return MIN_int32;
}

void FImgMediaSceneViewExtension::CacheViewInfo_NoLock(FSceneViewFamily& InViewFamily, const FSceneView& View)
{
	if (IsViewFamilyFilteredOut(InViewFamily))
	{
		return;
	}
	static const auto CVarMinAutomaticViewMipBiasOffset = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.ViewTextureMipBias.Offset"));
	static const auto CVarMinAutomaticViewMipBias = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.ViewTextureMipBias.Min"));
	const float FieldOfViewMultiplier = CVarImgMediaFieldOfViewMultiplier.GetValueOnGameThread();

	float ResolutionFraction = InViewFamily.SecondaryViewFraction;

	if (InViewFamily.GetScreenPercentageInterface())
	{
		DynamicRenderScaling::TMap<float> UpperBounds = InViewFamily.GetScreenPercentageInterface()->GetResolutionFractionsUpperBound();
		ResolutionFraction *= UpperBounds[GDynamicPrimaryResolutionFraction];
	}

	FImgMediaViewInfo Info;
	Info.Location = View.ViewMatrices.GetViewOrigin();
	Info.ViewDirection = View.GetViewDirection();
	Info.ViewProjectionMatrix = View.ViewMatrices.GetWorldToClip();
	
	// Use UnscaledViewRect (the AR-constrained rect from GetConstrainedViewRect): the projection
	// matrix is built for the camera's constrained AR - not UnconstrainedViewRect, which includes the
	// letterbox/pillarbox bars.
	{
		// Widen to int64 around the Scale() so an out-of-range result logs instead of crashing the
		// checked int32 narrowing.
		UE::Math::TIntRect<int64> ConstrainedViewportRect64(View.UnscaledViewRect);
		ConstrainedViewportRect64 = ConstrainedViewportRect64.Scale(ResolutionFraction);

		if (!IntFitsIn<int32>(ConstrainedViewportRect64.Min.X) ||
			!IntFitsIn<int32>(ConstrainedViewportRect64.Min.Y) ||
			!IntFitsIn<int32>(ConstrainedViewportRect64.Max.X) ||
			!IntFitsIn<int32>(ConstrainedViewportRect64.Max.Y))
		{
			UE_LOGF(LogImgMedia, Error, "Scaled view rect is out of bounds. Original UnscaledViewRect: Min: %d x %d, Max: %d x %d, Screen Percentage: %f",
				View.UnscaledViewRect.Min.X,
				View.UnscaledViewRect.Min.Y,
				View.UnscaledViewRect.Max.X,
				View.UnscaledViewRect.Max.Y,
				ResolutionFraction);
		}

		Info.ViewportRect = FIntRect(
			FIntPoint((int32)ConstrainedViewportRect64.Min.X, (int32)ConstrainedViewportRect64.Min.Y),
			FIntPoint((int32)ConstrainedViewportRect64.Max.X, (int32)ConstrainedViewportRect64.Max.Y)
		);
	}


	if (FMath::IsNearlyEqual(FieldOfViewMultiplier, 1.0f))
	{
		Info.OverscanViewProjectionMatrix = Info.ViewProjectionMatrix;
	}
	else
	{
		FMatrix AdjustedProjectionMatrix = View.ViewMatrices.GetViewToClip();

		const double HalfHorizontalFOV = FMath::Atan(1.0 / AdjustedProjectionMatrix.M[0][0]);
		const double HalfVerticalFOV = FMath::Atan(1.0 / AdjustedProjectionMatrix.M[1][1]);

		AdjustedProjectionMatrix.M[0][0] = 1.0 / FMath::Tan(HalfHorizontalFOV * FieldOfViewMultiplier);
		AdjustedProjectionMatrix.M[1][1] = 1.0 / FMath::Tan(HalfVerticalFOV * FieldOfViewMultiplier);

		Info.OverscanViewProjectionMatrix = View.ViewMatrices.GetWorldToView() * AdjustedProjectionMatrix;
	}

	// We store hidden or show-only ids to later avoid needless calculations when objects are not in view.
	if (View.ShowOnlyPrimitives.IsSet())
	{
		Info.bPrimitiveHiddenMode = false;
		Info.PrimitiveComponentIds = View.ShowOnlyPrimitives.GetValue();
	}
	else
	{
		Info.bPrimitiveHiddenMode = true;
		Info.PrimitiveComponentIds = View.HiddenPrimitives;
	}

	/* View.MaterialTextureMipBias is only set later in rendering so we replicate here the calculations
	 * found in FSceneRenderer::PreVisibilityFrameSetup.*/
	if (View.PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::TemporalUpscale)
	{
		const float EffectivePrimaryResolutionFraction = float(Info.ViewportRect.Width()) / (View.UnscaledViewRect.Width() * InViewFamily.SecondaryViewFraction);
		Info.MaterialTextureMipBias = -(FMath::Max(-FMath::Log2(EffectivePrimaryResolutionFraction), 0.0f)) + CVarMinAutomaticViewMipBiasOffset->GetValueOnGameThread();
		Info.MaterialTextureMipBias = FMath::Max(Info.MaterialTextureMipBias, CVarMinAutomaticViewMipBias->GetValueOnGameThread());

		if (!ensureMsgf(!FMath::IsNaN(Info.MaterialTextureMipBias) && FMath::IsFinite(Info.MaterialTextureMipBias), TEXT("Calculated material texture mip bias is invalid, defaulting to zero.")))
		{
			Info.MaterialTextureMipBias = 0.0f;
		}
	}
	else
	{
		Info.MaterialTextureMipBias = 0.0f;
	}

	// We cache the display resolution view info in case it's needed for compositing applications
	const bool bIsDisplayResolutionDifferent = !FMath::IsNearlyEqual(ResolutionFraction, InViewFamily.SecondaryViewFraction);
	if (bIsDisplayResolutionDifferent)
	{
		FImgMediaViewInfo PostUpscaleVirtualInfo = Info;
		PostUpscaleVirtualInfo.MaterialTextureMipBias = 0.0f;
		PostUpscaleVirtualInfo.ViewportRect = FIntRect(0, 0,
			FMath::CeilToInt(View.UnscaledViewRect.Width() * InViewFamily.SecondaryViewFraction),
			FMath::CeilToInt(View.UnscaledViewRect.Height() * InViewFamily.SecondaryViewFraction)
		);

		DisplayResolutionCachedViewInfos.Add(MoveTemp(PostUpscaleVirtualInfo));
	}

	CachedViewInfos.Add(MoveTemp(Info));
}

void FImgMediaSceneViewExtension::ResetViewInfoCache_NoLock()
{
	CachedViewInfos.Reset();
	DisplayResolutionCachedViewInfos.Reset();
}

#undef LOCTEXT_NAMESPACE
