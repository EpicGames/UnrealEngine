// Copyright Epic Games, Inc. All Rights Reserved.

#include "Loader/ImgMediaLoader.h"

#include "Algo/Reverse.h"
#include "BaseMediaSourceColorSettings.h"
#include "Containers/SortedMap.h"
#include "HAL/FileManager.h"
#include "Hash/Fnv.h"
#include "IImgMediaModule.h"
#include "ImgMediaGlobalCache.h"
#include "ImgMediaLoaderUtils.h"
#include "ImgMediaLoaderWork.h"
#include "ImgMediaPrivate.h"
#include "Misc/FrameRate.h"
#include "Misc/Paths.h"
#include "Misc/QueuedThreadPool.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"
#include "Player/ImgMediaTextureSample.h"
#include "Readers/DdsImgMediaReader.h"
#include "Readers/GenericImgMediaReader.h"
#include "Readers/IImgMediaReader.h"
#include "Readers/TmvImgMediaReader.h"
#include "Scheduler/ImgMediaScheduler.h"
#include "UObject/Class.h"

#if IMGMEDIA_EXR_SUPPORTED_PLATFORM
	#include "Readers/ExrImgMediaReader.h"
#endif

/** Time spent loading a new image sequence. */
DECLARE_CYCLE_STAT(TEXT("ImgMedia Loader Load Sequence"), STAT_ImgMedia_LoaderLoadSequence, STATGROUP_Media);

/** Time spent releasing cache in image loader destructor. */
DECLARE_CYCLE_STAT(TEXT("ImgMedia Loader Release Cache"), STAT_ImgMedia_LoaderReleaseCache, STATGROUP_Media);

constexpr int32 FImgMediaLoader::MAX_MIPMAP_LEVELS;

static TAutoConsoleVariable<int32> CVarImgMediaFrameInvalidationMaxCount(
	TEXT("ImgMedia.FrameInvalidationMaxCount"),
	2,
	TEXT("Maximum number of cached frames that can be invalidated when missing the latest mips/tiles."));

static TAutoConsoleVariable<int32> CVarImgMediaMaxFrameSequenceGap(
	TEXT("ImgMedia.MaxFrameSequenceGap"),
	10000,
	TEXT("Maximum number of frames that will be filled by empty frames in a sequence when there is a gap in the frame indices of the filenames."));

static TAutoConsoleVariable<bool> CVarImgMediaMipMapDebugEnable(
	TEXT("ImgMedia.MipMapDebug"),
	false,
	TEXT("Display debug on mipmaps and tiles used by the ImgMedia plugin.\n")
	TEXT("   0: off (default)\n")
	TEXT("   1: on\n"),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarImgMediaPausedRefreshEnabled(
	TEXT("ImgMedia.PausedRefresh"),
	true,
	TEXT("Enables player tile visibility refresh while paused.\n")
	TEXT("   0: off\n")
	TEXT("   1: on (default)\n"),
	ECVF_Default);

namespace ImgMediaLoader
{
	void CheckAndUpdateImgDimensions(FIntPoint& InOutSequenceDim, const FIntPoint& InNewDim)
	{
		if (InOutSequenceDim != InNewDim)
		{
			// This means that image sequence has inconsistent dimension and user needs to be made aware.
			UE_LOGF(LogImgMedia, Warning, "Image sequence has inconsistent dimensions. The original sequence dimension (%dx%d) is changed to the new dimension (%dx%d).",
				InOutSequenceDim.X, InOutSequenceDim.Y, InNewDim.X, InNewDim.Y);

			InOutSequenceDim = InNewDim;
		}
	}

	// Check if [CurrentFrame] is contained in the [OffsetCount] number of frames after [OriginFrame], also taking into account looping and play direction.
	bool IsCachedFrameInRange(int32 CurrentFrame, int32 OriginFrame, int32 OffsetCount, int32 TotalNumFrames, float PlayRate, bool bInIsPaused)
	{
		if (OffsetCount <= 0)
		{
			return false;
		}

		ensure(CurrentFrame >= 0 && CurrentFrame < TotalNumFrames);
		ensure(OriginFrame >= 0 && OriginFrame < TotalNumFrames);

		const int32 PlayRateSign = (PlayRate >= 0.0f) ? 1 : -1;

		// Unless the player is paused, we start one frame away, since it's generally too late to enqueue additional
		// update work for the current cached frame.
		const int32 StartOffset = bInIsPaused ? 0 : 1;

		for (int32 Offset = StartOffset; Offset <= OffsetCount; ++Offset)
		{
			int32 OffsetFrame = FMath::WrapExclusive(OriginFrame + PlayRateSign * Offset, 0, TotalNumFrames);
			if (OffsetFrame == CurrentFrame)
			{
				return true;
			}
		}

		return false;
	}
}

FImgMediaLoaderBandwidth::FImgMediaLoaderBandwidth()
	: Current(0)
	, Effective(0)
	, Required(0)
	, ReadTimeCache()
	, ReadTimeCacheIndex(0)
{

}

void FImgMediaLoaderBandwidth::Update(const TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>& Frame, double WorkTime)
{
	static constexpr int32 READ_TIME_CACHE_MAX = 64;

	const FImgMediaFrameInfo FrameInfo = Frame->GetInfo();
	// Calculate the current uncompressed bandwidth
	SIZE_T BytesLoaded = FrameInfo.UncompressedSize;
	Current = (float)BytesLoaded / WorkTime;

	if (FrameInfo.bHasTiles)
	{
		int32 TotalNumTiles = 0;

		for (int32 MipLevel = 0; MipLevel < FrameInfo.NumMipLevels; ++MipLevel)
		{
			const int32 MipLevelDiv = 1 << MipLevel;
			int32 NumTilesX = FMath::Max(1, FMath::CeilToInt(float(FrameInfo.NumTiles.X) / MipLevelDiv));
			int32 NumTilesY = FMath::Max(1, FMath::CeilToInt(float(FrameInfo.NumTiles.Y) / MipLevelDiv));
			TotalNumTiles += NumTilesX * NumTilesY;
		}

		// Adjusted based on how many tiles were loaded
		Current *= (float)Frame->NumTilesRead / FMath::Max(1, TotalNumTiles);
	}

	// Store read times with their respectivate timestamps...
	if (ReadTimeCache.Num() == READ_TIME_CACHE_MAX)
	{
		ReadTimeCache[ReadTimeCacheIndex] = TPairInitializer(FPlatformTime::Seconds(), WorkTime);
		ReadTimeCacheIndex = (ReadTimeCacheIndex + 1) % READ_TIME_CACHE_MAX;
	}
	else
	{
		ReadTimeCache.Emplace(FPlatformTime::Seconds(), WorkTime);
	}

	// Calculate the effective/utilized bandwidth as the fraction of the elapsed time spent reading.
	double IntervalStart = TNumericLimits<double>::Max();
	double IntervalEnd = TNumericLimits<double>::Lowest();
	double TotalReadTime = 0.0;

	for (const TPair<double, double>& Pair : ReadTimeCache)
	{
		double CachedTimestamp = Pair.Key;
		double CachedReadTime = Pair.Value;
		IntervalStart = FMath::Min(IntervalStart, CachedTimestamp - CachedReadTime);
		IntervalEnd = FMath::Max(IntervalEnd, CachedTimestamp);
		TotalReadTime += CachedReadTime;
	}

	Effective = (TotalReadTime / (IntervalEnd - IntervalStart)) * Current;
}

void FImgMediaLoaderBandwidth::EmptyCache()
{
	ReadTimeCache.Empty(ReadTimeCache.Max());
	ReadTimeCacheIndex = 0;
}

/* FImgMediaLoader structors
 *****************************************************************************/

FImgMediaLoader::FImgMediaLoader(const TSharedRef<FImgMediaScheduler, ESPMode::ThreadSafe>& InScheduler,
	const TSharedRef<FImgMediaGlobalCache, ESPMode::ThreadSafe>& InGlobalCache,
	const TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe>& InSourceColorSettings,
	bool bInFillGapsInSequence,
	const FString& InLayerName,
	const FImgMediaLoaderSmartCacheSettings& InSmartCacheSettings,
	IMediaEventSink* InEventSink,
	const FGuid& InPlayerGuid)
	: Frames(1)
	, Initialized(false)
	, bFillGapsInSequence(bInFillGapsInSequence)
	, TilingDescription()
	, NumLoadAhead(0)
	, NumLoadBehind(0)
	, NumFramesToLoad(0)
	, NumMipLevels(1)
	, NumBytesPerPixel(0)
	, Scheduler(InScheduler)
	, GlobalCache(InGlobalCache)
	, TileVisibilityResolver(MakeShared<FImgMediaTileVisibilityResolver, ESPMode::ThreadSafe>())
	, SourceColorSettings(InSourceColorSettings)
	, LayerName(InLayerName)
	, SequenceDim(FIntPoint::ZeroValue)
	, SequenceDuration(FTimespan::Zero())
	, SequenceFrameRate(0, 0)
	, LastRequestedFrame(INDEX_NONE)
	, EventSink(InEventSink)
	, PlayerGuid(InPlayerGuid)
	, RetryCount(0)
	, NextFrameNumberJobID(0)
	, UseGlobalCache(false)
	, SmartCacheSettings(InSmartCacheSettings)
	, bIsPlaybackBlocking(false)
	, bIsBandwidthThrottlingEnabled(true)
	, bIsSkipFramesEnabled(false)
	, SkipFramesCounter(0)
	, SkipFramesLevel(0)
#if WITH_EDITOR
	, Bandwidth()
#endif
{
	ResetFetchLogic();
	QueuedSampleFetch.LastTimeStamp.SetTimeAndIndexValue(FTimespan::Zero(), 0);
	QueuedSampleFetch.LastDuration = FTimespan::Zero();

	UE_LOGF(LogImgMedia, Verbose, "Loader %p [%ls]: Created", this, *PlayerGuid.ToString());
}


FImgMediaLoader::~FImgMediaLoader()
{
	UE_LOGF(LogImgMedia, Verbose, "Loader %p [%ls]: Destroyed", this, *PlayerGuid.ToString());

	// clean up work item pool
	for (auto Work : WorkPool)
	{
		delete Work;
	}

	WorkPool.Empty();

	// release cache
	{
		SCOPE_CYCLE_COUNTER(STAT_ImgMedia_LoaderReleaseCache);

		Frames.Empty();
		PendingFrameNumbers.Empty();
	}
}

void FImgMediaLoader::TickFrame()
{
	if (!TileVisibilityResolver.IsValid())
	{
		return;
	}

	// Debug overlay reads the snapshot used by the previous frame's workers.
	// Invalidate after so the next frame's first GetSnapshot() rebuilds.
	if (CVarImgMediaMipMapDebugEnable.GetValueOnGameThread())
	{
		TileVisibilityResolver->AddVisibilitySnapshotDebugOverlay();
	}
	TileVisibilityResolver->InvalidateSnapshot();

	// Paused-Refresh: drive Update here to detect tile visibility changes and trigger refresh.
	// Note: a 1-frame sequence is functionally paused, so it will go through the paused refresh path.
	const bool bShouldRefresh = bIsPaused || GetNumImages() == 1;
	if (bShouldRefresh && LastRequestedFrame != INDEX_NONE && CVarImgMediaPausedRefreshEnabled.GetValueOnGameThread())
	{
		Update(LastRequestedFrame, 0.0f, LastRequestedLoop);
	}
}


/* FImgMediaLoader interface
 *****************************************************************************/

uint64 FImgMediaLoader::GetBitRate() const
{
	FScopeLock Lock(&CriticalSection);
	return SequenceDim.X * SequenceDim.Y * NumBytesPerPixel * 8 * SequenceFrameRate.AsDecimal();
}


void FImgMediaLoader::GetBusyTimeRanges(TRangeSet<FTimespan>& OutRangeSet) const
{
	FScopeLock Lock(&CriticalSection);
	TArray<int32> FrameNumbers;
	FrameNumbers.Reserve(QueuedFrameNumbers.Num());
	for (const auto& Entry : QueuedFrameNumbers)
	{
		FrameNumbers.Push(Entry.Frame);
	}
	FrameNumbersToTimeRanges(FrameNumbers, OutRangeSet);
}


void FImgMediaLoader::GetCompletedTimeRanges(TRangeSet<FTimespan>& OutRangeSet) const
{
	FScopeLock Lock(&CriticalSection);

	TArray<int32> CompletedFrames;
	if (UseGlobalCache)
	{
		GlobalCache->GetIndices(SequenceName, CompletedFrames);
	}
	else
	{
		Frames.GetKeys(CompletedFrames);
	}
	FrameNumbersToTimeRanges(CompletedFrames, OutRangeSet);
}

const FString& FImgMediaLoader::GetImagePath(int32 FrameNumber, int32 MipLevel) const
{
	if ((MipLevel < 0) || (MipLevel >= ImagePaths.Num() ||
		(FrameNumber < 0) || (FrameNumber >= ImagePaths[MipLevel].Num())))
	{
		UE_LOGF(LogImgMedia, Error, "Loader %p: GetImagePath has wrong parameters, FrameNumber:%d MipLevel:%d.",
			this, FrameNumber, MipLevel);
		static FString EmptyString(TEXT(""));
		return EmptyString;
	}
	else
	{
		return ImagePaths[MipLevel][FrameNumber];
	}
}


void FImgMediaLoader::ResetFetchLogic()
{
	QueuedSampleFetch.LoopIndex = 0;

	SkipFramesCounter = 0;
	SkipFramesLevel = 0;
}

bool FImgMediaLoader::HasMipsOrTilesEnabled() const
{
	const bool bHasResolverProviders = TileVisibilityResolver.IsValid() && TileVisibilityResolver->HasProviders();
	return bHasResolverProviders && (NumMipLevels > 1 || IsTiled());
}

void FImgMediaLoader::SetIsPlaybackBlocking(bool bIsBlocking)
{
	bIsPlaybackBlocking = bIsBlocking;
	UpdateBandwidthThrottling();
}

void FImgMediaLoader::SetIsPaused(bool bInIsPaused)
{
	const bool bWasPaused = bIsPaused.exchange(bInIsPaused, std::memory_order_relaxed);
#if WITH_EDITOR
	if (bInIsPaused && !bWasPaused)
	{
		// Rolling-window measurements across a pause boundary are meaningless.
		Bandwidth.EmptyCache();
	}
#endif
}

void FImgMediaLoader::Flush()
{
	ResetFetchLogic();
	// Adjust last timestamp / play head so it is within the current, reset sequence index range
	QueuedSampleFetch.LastTimeStamp.SetSequenceIndex(0);
	QueuedSampleFetch.LastTimeStamp.SetLoopIndex(0);
}

void FImgMediaLoader::Seek(const FMediaTimeStamp& SeekTarget, bool bReverse)
{
	ResetFetchLogic();
	QueuedSampleFetch.LastTimeStamp.SetTimeAndIndexValue(SeekTarget.Time, SeekTarget.GetIndexValue());
	QueuedSampleFetch.LastDuration = FTimespan::Zero();
}


bool FImgMediaLoader::DiscardVideoSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bIsLoopingEnabled, float PlayRate)
{
	bool bOk = false;

	check(QueuedSampleFetch.LastTimeStamp.IsValid());

	if (PlayRate >= 0.0f)
	{
		if (QueuedSampleFetch.LastTimeStamp < TimeRange.GetUpperBoundValue())
		{
			QueuedSampleFetch.LastTimeStamp = TimeRange.GetUpperBoundValue();
			bOk = true;
		}
	}
	else
	{
		if (QueuedSampleFetch.LastTimeStamp >= TimeRange.GetLowerBoundValue())
		{
			QueuedSampleFetch.LastTimeStamp = TimeRange.GetLowerBoundValue() - FTimespan::FromSeconds(SequenceFrameRate.AsInterval());
			bOk = true;
		}
	}

	if (bOk)
	{
		ResetFetchLogic();
		QueuedSampleFetch.LastDuration = FTimespan::Zero();
	}

	return bOk;
}


IMediaSamples::EFetchBestSampleResult FImgMediaLoader::FetchBestVideoSampleForTimeRange(const TRange<FMediaTimeStamp>& InTimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample, bool bIsLoopingEnabled, float PlayRate)
{
	if (IsInitialized() && InTimeRange.HasLowerBound() && InTimeRange.HasUpperBound())
	{
		// A few sanity checks
		check(InTimeRange.GetLowerBoundValue() <= InTimeRange.GetUpperBoundValue());
		check(InTimeRange.GetLowerBoundValue().GetSequenceIndex() == InTimeRange.GetUpperBoundValue().GetSequenceIndex());
		check(QueuedSampleFetch.LastTimeStamp.IsValid());

		auto TimeRange = InTimeRange;

		// To avoid odd behavior with out of range time values, clamp them down just in case
		if (!bIsLoopingEnabled)
		{
			TimeRange.SetLowerBoundValue(FMediaTimeStamp(FMath::Clamp(TimeRange.GetLowerBoundValue().Time, FTimespan::Zero(), SequenceDuration), TimeRange.GetLowerBoundValue().GetIndexValue()));
			TimeRange.SetUpperBoundValue(FMediaTimeStamp(FMath::Clamp(TimeRange.GetUpperBoundValue().Time, FTimespan::Zero(), SequenceDuration), TimeRange.GetUpperBoundValue().GetIndexValue()));
		}
		FTimespan StartTime = TimeRange.GetLowerBoundValue().Time;
		int32 StartLoopIndex = TimeRange.GetLowerBoundValue().GetLoopIndex();
		FTimespan EndTime = TimeRange.GetUpperBoundValue().Time;
		int32 EndLoopIndex;

		// The end time is EXCLUSIVE, move the time by the smallest amount we can so we correctly select indices of frames
		if (EndTime > FTimespan::Zero())
		{
			EndTime -= FTimespan(1);
			EndLoopIndex = TimeRange.GetUpperBoundValue().GetLoopIndex();
		}
		else
		{
			EndTime = SequenceDuration - FTimespan(1);
			EndLoopIndex = TimeRange.GetUpperBoundValue().GetLoopIndex() - 1;
		}

		int32 NumFrames = GetNumImages();
		FTimespan FrameLength = FTimespan::FromSeconds(SequenceFrameRate.AsInterval());

		FTimespan UnwrappedStartTime = StartTime + FrameLength * NumFrames * StartLoopIndex;
		FTimespan UnwrappedEndTime = EndTime + FrameLength * NumFrames * EndLoopIndex;
		int64 UnwrappedStartIndex = TimeToFrameNumberUnbound(UnwrappedStartTime);
		int64 UnwrappedEndIndex = TimeToFrameNumberUnbound(UnwrappedEndTime);
		FTimespan UnwrappedStartTimeAligned = UnwrappedStartIndex * FrameLength;
		FTimespan UnwrappedEndTimeAligned = UnwrappedEndIndex * FrameLength;

		int64 MaxIdx;
		if (PlayRate >= 0.0f)
		{
			// Forward...

			// More than one possibility?
			if (UnwrappedStartIndex != UnwrappedEndIndex)
			{
				FTimespan FirstFrameDuration = FrameLength - (UnwrappedStartTime - UnwrappedStartTimeAligned);
				FTimespan LastFrameDuration = UnwrappedEndTime - UnwrappedEndTimeAligned;

				FTimespan MaxFrameDuration;
				if ((UnwrappedEndIndex - UnwrappedStartIndex) > 1)
				{
					MaxIdx = UnwrappedEndIndex - 1;
					MaxFrameDuration = FrameLength;
				}
				else
				{
					MaxIdx = UnwrappedStartIndex;
					MaxFrameDuration = FirstFrameDuration;
				}
				if (LastFrameDuration >= MaxFrameDuration)
				{
					MaxIdx = UnwrappedEndIndex;
					MaxFrameDuration = LastFrameDuration;
				}
			}
			else
			{
				MaxIdx = UnwrappedEndIndex;
			}
		}
		else
		{
			// Backward...

			// More than one possibility?
			if (UnwrappedStartIndex != UnwrappedEndIndex)
			{
				FTimespan FirstFrameDuration = FrameLength - (UnwrappedStartTime - UnwrappedStartTimeAligned);
				FTimespan LastFrameDuration = UnwrappedEndTime - UnwrappedEndTimeAligned;

				FTimespan MaxFrameDuration;
				if ((UnwrappedEndIndex - UnwrappedStartIndex) > 1)
				{
					MaxIdx = UnwrappedStartIndex + 1;
					MaxFrameDuration = FrameLength;
				}
				else
				{
					MaxIdx = UnwrappedEndIndex;
					MaxFrameDuration = LastFrameDuration;
				}
				if (FirstFrameDuration >= MaxFrameDuration)
				{
					MaxIdx = UnwrappedStartIndex;
					MaxFrameDuration = FirstFrameDuration;
				}
			}
			else
			{
				MaxIdx = UnwrappedStartIndex;
			}
		}

		// - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - -

		// Anything?
		int32 MaxLoopIdx;
		if (bIsLoopingEnabled)
		{
			MaxLoopIdx = int32(MaxIdx / NumFrames);
			bool bNegative = (MaxIdx < 0);
			MaxIdx = bNegative ? -MaxIdx : MaxIdx;
			MaxIdx = MaxIdx - (MaxIdx / NumFrames) * NumFrames;
			if (bNegative)
			{
				MaxIdx = (NumFrames - 1) - MaxIdx;
			}
		}
		else
		{
			MaxLoopIdx = 0;	// ignored below
			// Clamp this between [0..N-1], but keep a look out for any loop index that might persist from an earlier looping state
			MaxIdx = FMath::Clamp(MaxIdx, StartLoopIndex * NumFrames, StartLoopIndex * NumFrames + NumFrames - 1) - StartLoopIndex * NumFrames;
		}

		const TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>* Frame;

		// Throttling only caches skip-grid-aligned frames (see Update()/GetSkipFrame()), so
		// align the request to that grid or GetCachedFrame() misses every off-grid index.
		int32 FrameStride = 1;
		if (SkipFramesLevel > 0)
		{
			FrameStride = 1 << SkipFramesLevel;
			MaxIdx = GetSkipFrame(MaxIdx);
		}

		// Request data for the frame we would like... (in case it's not requested or in, yet)
		RequestFrame(MaxIdx, PlayRate, bIsLoopingEnabled);

		FScopeLock Lock(&CriticalSection);

		// Get a frame if we have one available right now...
		Frame = GetCachedFrame(MaxIdx);

		// Exact clock frame not cached? Substitute the nearest one - but never while blocking,
		// where the facade needs the exact frame (or NoSample, to block until it loads).
		if ((!bIsPlaybackBlocking) && ((Frame == nullptr) || ((*Frame).IsValid() == false)))
		{
			// Returning NoSample here freezes delivery, letting the facade clock (and our look-ahead
			// target) run away from the displayed position - the "stuck far behind loading" symptom.
			// Catch up instead: deliver the newest cached frame on the trailing side of the clock,
			// dropping un-loaded frames, without regressing past the last shown frame. Compare on the
			// unwrapped timeline (loop*NumFrames + index) so a fresh loop's low indices still rank as
			// newer across the loop point.
			const bool bHavePrevious = !QueuedSampleFetch.LastDuration.IsZero();
			const int64 LastDeliveredUnwrapped = bHavePrevious
				? (int64)QueuedSampleFetch.LastTimeStamp.GetLoopIndex() * NumFrames + (int32)TimeToFrameNumber(QueuedSampleFetch.LastTimeStamp.Time)
				: 0;
			const int32 SearchLimit = FMath::Max(NumFramesToLoad, 1);
			const int32 Step = (PlayRate >= 0.0f) ? -1 : 1;

			int32 ProbeIdx = (int32)MaxIdx + Step;
			for (int32 Count = 0; (Count < SearchLimit) && (ProbeIdx >= 0) && (ProbeIdx < NumFrames); ++Count, ProbeIdx += Step)
			{
				const TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>* ProbeFrame = GetCachedFrame(ProbeIdx);
				if ((ProbeFrame != nullptr) && (*ProbeFrame).IsValid())
				{
					// Newest cached frame on the trailing side (shares the clock loop, MaxLoopIdx);
					// use it only if it advances playback.
					const int64 ProbeUnwrapped = (int64)MaxLoopIdx * NumFrames + ProbeIdx;
					const bool bAdvances = !bHavePrevious ||
						((PlayRate >= 0.0f) ? (ProbeUnwrapped > LastDeliveredUnwrapped) : (ProbeUnwrapped < LastDeliveredUnwrapped));
					if (bAdvances)
					{
						Frame = ProbeFrame;
						MaxIdx = ProbeIdx;
					}
					break;
				}
			}
		}

		// Got a potential frame?
		if (Frame && (*Frame).IsValid())
		{
			// Yes.

			/*
				The player facade will pass in a time range with current primary and secondary sequence indices.
				We need to make sure that we detect if we return a value "before" the last one (loop case) and
				adjust the secondary index accordingly.
			*/

			RetryCount = 0;

			FTimespan SampleTime = FrameNumberToTime(MaxIdx);

			// Set loop index if looping is active (otherwise we keep what was known before)
			if (bIsLoopingEnabled && !QueuedSampleFetch.LastDuration.IsZero())
			{
				QueuedSampleFetch.LoopIndex = MaxLoopIdx;
			}

			// Track state & setup sample for caller...
			QueuedSampleFetch.LastTimeStamp.SetTime(SampleTime);
			QueuedSampleFetch.LastTimeStamp.SetSequenceIndex(TimeRange.GetLowerBoundValue().GetSequenceIndex());
			QueuedSampleFetch.LastTimeStamp.SetLoopIndex(QueuedSampleFetch.LoopIndex);
			// A skipped sample covers its whole collapsed bucket (FrameStride frames) so
			// consecutive deliveries tile contiguously and the facade timing stays stable.
			QueuedSampleFetch.LastDuration = FrameLength * FrameStride;

			// We are clear to return it as new result... Make a sample & initialize it.
			const TSharedPtr<FImgMediaTextureSample, ESPMode::ThreadSafe> Sample = Reader->CreateSample();

			ImgMediaLoader::CheckAndUpdateImgDimensions(SequenceDim, Frame->Get()->GetDim());

			Sample->Initialize(*Frame->Get(), SequenceDim, QueuedSampleFetch.LastTimeStamp, QueuedSampleFetch.LastDuration, GetNumMipLevels(), TilingDescription, SourceColorSettings);

			OutSample = Sample;
			CSV_EVENT(ImgMedia, TEXT("LoaderFetchHit %" INT64_FMT " %" INT64_FMT "-%" INT64_FMT), MaxIdx, UnwrappedStartIndex% NumFrames, UnwrappedEndIndex% NumFrames);
			return IMediaSamples::EFetchBestSampleResult::Ok;
		}
		else
		{
			// We did not get a frame...
			// Could we have lost a frame that we previously loaded due to the global cache being full?
			if (UseGlobalCache)
			{
				HandleGlobalCacheFrameEviction();
			}
		}
		CSV_EVENT(ImgMedia, TEXT("LoaderFetchMiss %" INT64_FMT "-%" INT64_FMT), UnwrappedStartIndex % NumFrames, UnwrappedEndIndex % NumFrames);
	}
	return IMediaSamples::EFetchBestSampleResult::NoSample;
}


bool FImgMediaLoader::PeekVideoSampleTime(FMediaTimeStamp &TimeStamp, bool bIsLoopingEnabled, float PlayRate, const FTimespan& CurrentTime)
{
	// When playing (i.e not seeking), we don't actually return a "frame" here as we do NOT involve any requests to the cache,
	// we simply assume we could deliver what a "queue" would contain on demand later (in FetchBestVideoSampleForTimeRange).
	// However, when seeking, we need to check the cache here so the seek status properly reflect the state of the cache and allow
	// for seek throttling to work as intended.
	if (IsInitialized())
	{
		check(QueuedSampleFetch.LastTimeStamp.IsValid());

		if (PlayRate >= 0.0f)
		{
			TimeStamp = QueuedSampleFetch.LastTimeStamp + QueuedSampleFetch.LastDuration;
			while(TimeStamp.Time >= SequenceDuration)
			{
				TimeStamp.Time -= SequenceDuration;
				TimeStamp.AdjustLoopIndex(1);
			}
		}
		else
		{
			TimeStamp = QueuedSampleFetch.LastTimeStamp - QueuedSampleFetch.LastDuration;
			while(TimeStamp.Time.GetTicks() < 0)
			{
				TimeStamp.Time += SequenceDuration;
				TimeStamp.AdjustLoopIndex(-1);
			}
		}

		// If waiting for a seek and there is no other playback blocking,
		// properly indicate if the sample has arrived to allow for seek throttling.
		if (QueuedSampleFetch.LastDuration.IsZero() && !bIsPlaybackBlocking)
		{
			const uint32 PeekFrameNumber = TimeToFrameNumber(TimeStamp.Time);

			bool bHasFrame;
			if (UseGlobalCache)
			{
				bHasFrame = GlobalCache->Contains(SequenceName, PeekFrameNumber);
			}
			else
			{
				FScopeLock ScopeLock(&CriticalSection);
				bHasFrame = Frames.Contains(PeekFrameNumber);
			}

			if (!bHasFrame)
			{
				// Request the missing frame to avoid blocking.
				RequestFrame(PeekFrameNumber, PlayRate, bIsLoopingEnabled);

				// We did not get a frame...
				// Could we have lost a frame that we previously loaded due to the global cache being full?
				if (UseGlobalCache)
				{
					FScopeLock Lock(&CriticalSection);
					HandleGlobalCacheFrameEviction();
				}

				// Need to invalidate the time stamp because FMediaPlayerFacade::UpdateSeekStatus ignores the return value.
				TimeStamp.Invalidate();
			}
			return bHasFrame;
		}

		return true;
	}
	return false;
}


bool FImgMediaLoader::IsFrameFirst(const FTimespan& TimeStamp) const
{
	return TimeToFrameNumber(TimeStamp) == 0;
}


bool FImgMediaLoader::IsFrameLast(const FTimespan & TimeStamp) const
{
	return TimeToFrameNumber(TimeStamp) == (GetNumImages() - 1);
}


void FImgMediaLoader::GetPendingTimeRanges(TRangeSet<FTimespan>& OutRangeSet) const
{
	FScopeLock Lock(&CriticalSection);
	FrameNumbersToTimeRanges(PendingFrameNumbers, OutRangeSet);
}


IQueuedWork* FImgMediaLoader::GetWork()
{
	FScopeLock Lock(&CriticalSection);

	if (PendingFrameNumbers.IsEmpty())
	{
		return nullptr;
	}

	int32 FrameNumber = PendingFrameNumbers.Pop(EAllowShrinking::No);

	TMap<int32, FMediaTileSelection> DesiredMipsAndTiles;
	GetDesiredMipTiles(FrameNumber, DesiredMipsAndTiles);

	if (DesiredMipsAndTiles.IsEmpty())
	{
		// Still provide the cache with an empty frame to prevent blocking playback from stalling.
		TryAddEmptyFrame(FrameNumber);

		// No selection was visible, so we don't queue any work.
		return nullptr;
	}

	// Update minimum mip level to upscale
	MinimumMipLevelToUpscale = GetDesiredMinimumMipLevelToUpscale();

	if (MinimumMipLevelToUpscale >= 0 && NumMipLevels <= 1)
	{
		UE_CALL_ONCE([ImagePath = GetImagePath(0, 0)]
			{
				UE_LOGF(LogImgMedia, Display, "No upscaling for sequence without mips: %ls", *ImagePath);
			});
	}

	FImgMediaLoaderWork* Work = (WorkPool.Num() > 0) ? WorkPool.Pop() : new FImgMediaLoaderWork(AsShared(), Reader.ToSharedRef());

	// Get the existing frame so we can add the mip level to it.
	TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> ExistingFrame;
	if (const TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>* ExistingFramePtr = GetCachedFrame(FrameNumber))
	{
		ExistingFrame = *ExistingFramePtr;
	}

	// Set up work.
	Work->Initialize(NextFrameNumberJobID, FrameNumber, DesiredMipsAndTiles, ExistingFrame);
	QueuedFrameNumbers.Emplace(FrameNumber, NextFrameNumberJobID++);

	return Work;
}


void FImgMediaLoader::Initialize(const FString& SequencePath, const FFrameRate& FrameRateOverride,
	bool Loop, bool bIsSequencerEnvironment)
{
	UE_LOGF(LogImgMedia, Verbose, "Loader %p: Initializing with %ls (FrameRateOverride = %ls, Loop = %i)",
		this,
		*SequencePath,
		*FrameRateOverride.ToPrettyText().ToString(),
		Loop
	);

	check(!Initialized); // reinitialization not allowed for now

	FImgMediaFrameInfo FirstFrameInfo;

	if (LoadSequence(SequencePath, FrameRateOverride, Loop, FirstFrameInfo))
	{
		WarmupSequence(FirstFrameInfo, Loop, bIsSequencerEnvironment);
	}

	FPlatformMisc::MemoryBarrier();

	Initialized = true;
}


bool FImgMediaLoader::RequestFrame(FTimespan Time, float PlayRate, bool Loop)
{
	return RequestFrame(TimeToFrameNumber(Time), PlayRate, Loop);
}


bool FImgMediaLoader::RequestFrame(int32 FrameNumber, float PlayRate, bool Loop)
{
	LastRequestedLoop = Loop;

	if ((FrameNumber == INDEX_NONE) || (FrameNumber == LastRequestedFrame))
	{
		// Make sure we call the reader even if we do no update - just in case it does anything
		Reader->OnTick();

		UE_LOGF(LogImgMedia, VeryVerbose, "Loader %p: Skipping frame %i for time %ls last %d", this, FrameNumber, *FrameNumberToTime(FrameNumber).ToString(TEXT("%h:%m:%s.%t")), LastRequestedFrame);
		return false;
	}

	UE_LOGF(LogImgMedia, VeryVerbose, "Loader %p: Requesting frame %i for time %ls", this, FrameNumber, *FrameNumberToTime(FrameNumber).ToString(TEXT("%h:%m:%s.%t")));

	Update(FrameNumber, PlayRate, Loop);
	LastRequestedFrame = FrameNumber;

	return true;
}

void FImgMediaLoader::HandleGlobalCacheFrameEviction()
{
	// Did we get a frame previously?
	if (LastRequestedFrame != INDEX_NONE)
	{
		// Are we loading any frames?
		if ((PendingFrameNumbers.Num() == 0) && (QueuedFrameNumbers.Num() == 0))
		{
			// Nope...
			// Wait for this to happen for one more frame.
			// If we have a 1 image sequence, we might have just missed the frame
			// so try again next time.
			RetryCount++;
			if (RetryCount > 1)
			{
				UE_LOGF(LogImgMedia, Error, "Reloading frames. The global cache may be too small.");
				LastRequestedFrame = INDEX_NONE;
				RetryCount = 0;
			}
		}
	}
}

/* FImgMediaLoader implementation
 *****************************************************************************/

void FImgMediaLoader::FrameNumbersToTimeRanges(const TArray<int32>& FrameNumbers, TRangeSet<FTimespan>& OutRangeSet) const
{
	if (!SequenceFrameRate.IsValid() || (SequenceFrameRate.Numerator <= 0))
	{
		return;
	}

	for (const auto& FrameNumber : FrameNumbers)
	{
		const FTimespan FrameStartTime = FrameNumberToTime(FrameNumber);
		const FTimespan NextStartTime = FrameNumberToTime(FrameNumber + 1);

		OutRangeSet.Add(TRange<FTimespan>(FrameStartTime, NextStartTime));
	}
}


bool FImgMediaLoader::LoadSequence(const FString& SequencePath, const FFrameRate& FrameRateOverride, bool Loop, FImgMediaFrameInfo& FirstFrameInfo)
{
	SCOPE_CYCLE_COUNTER(STAT_ImgMedia_LoaderLoadSequence);

	if (SequencePath.IsEmpty())
	{
		return false;
	}

	// Check if SequencePath is a container file (e.g. .tmv) rather than a directory.
	bIsContainerFile = FTmvImgMediaReader::IsContainerFileExtensionSupported(FPaths::GetExtension(SequencePath))
		&& FPaths::FileExists(SequencePath);

	TSharedPtr<FTmvImgMediaReader, ESPMode::ThreadSafe> ContainerReader;

	if (bIsContainerFile)
	{
		// Container file: create the reader first, get frame count from the demuxer track info.
		ContainerReader = FTmvImgMediaReader::CreateReader(AsShared(), SequencePath);
		if (!ContainerReader.IsValid())
		{
			UE_LOGF(LogImgMedia, Error, "Failed to create container reader for %ls", *SequencePath);
			return false;
		}

		const FTmvMediaDemuxerTrackInfo* VideoTrackInfo = ContainerReader->GetVideoTrackInfo();

		if (!VideoTrackInfo)
		{
			UE_LOGF(LogImgMedia, Error, "Container reader for %ls has no video track.", *SequencePath);
			return false;
		}
		
		NumContainerFrames = static_cast<int32>(VideoTrackInfo->NumSamples);

		if (NumContainerFrames <= 0)
		{
			UE_LOGF(LogImgMedia, Error, "Container %ls has no frames", *SequencePath);
			return false;
		}
	}
	else
	{
		// File sequence: scan directory for individual frame files.
		TArray<FString> FoundPaths;
		FindFiles(SequencePath, FoundPaths);
		if (FoundPaths.Num() == 0)
		{
			UE_LOGF(LogImgMedia, Error, "The directory %ls does not contain any image files", *SequencePath);
			return false;
		}
		ImagePaths.Emplace(MoveTemp(FoundPaths));

		// Get mips.
		FindMips(SequencePath);
	}

	// Default to a 1x1 tile
	TilingDescription.TileNum = FIntPoint(1, 1);

	FScopeLock Lock(&CriticalSection);
	
	Reader = ContainerReader;

	// create image reader (if not already created for container path)
	if (!Reader.IsValid())
	{
		const FString FirstExtension = FPaths::GetExtension(ImagePaths[0][0]);

		if (FirstExtension == TEXT("exr"))
		{
#if IMGMEDIA_EXR_SUPPORTED_PLATFORM
			// Differentiate between Uncompressed exr and the rest.
			Reader = FExrImgMediaReader::GetReader(AsShared(), ImagePaths[0][0], LayerName);
#else
			UE_LOGF(LogImgMedia, Error, "EXR image sequences are currently supported on macOS, Linux and Windows only");
			return false;
#endif
		}
		else if (FirstExtension == TEXT("dds"))
		{
			Reader = MakeShared<FDdsImgMediaReader>(AsShared());
		}
		else if (FTmvImgMediaReader::IsImageFileExtensionSupported(FirstExtension))
		{
			Reader = FTmvImgMediaReader::CreateReader(AsShared(), ImagePaths[0][0]);
		}
		else
		{
			Reader = MakeShareable(new FGenericImgMediaReader(AsShared()));
		}

		if (Reader.IsValid() == false)
		{
			UE_LOGF(LogImgMedia, Error, "Reader is not valid for %ls.", *ImagePaths[0][0]);
			return false;
		}
	}

	const UImgMediaSettings* Settings = GetDefault<UImgMediaSettings>();
	bIsBandwidthThrottlingEnabled = Settings->BandwidthThrottlingEnabled;
	UpdateBandwidthThrottling();
	UseGlobalCache = Settings->UseGlobalCache;
	if (SmartCacheSettings.bIsEnabled)
	{
		// Smart cache does not use the global cache.
		UseGlobalCache = false;
	}
	/** Include LayerName in the cache key so that different layer selections
	 *  for the same sequence path map to separate cache buckets. Empty LayerName
	 *  uses the path alone, preserving the existing key for the default case.
	 * 
	 *  The layer name is stored as its hash rather than the raw string because
	 *  FName comparison is case-insensitive: embedding a uint32 keeps the key
	 *  case-sensitive (numbers are unaffected by FName's case folding).
	 */
	SequenceName = LayerName.IsEmpty()
		? FName(*SequencePath)
		: FName(*FString::Printf(TEXT("%s?layer=%u"), *SequencePath, UE::HashStringFNV1a32(MakeStringView(LayerName))));

	// fetch sequence attributes from first image
	{
		const TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>* CachedFrame = GetCachedFrame(0);

		// Read info if the first frame is cached and not empty
		if (CachedFrame && !(*CachedFrame)->MipTilesPresent.IsEmpty())
		{
			FirstFrameInfo = CachedFrame->Get()->GetInfo();
		}
		else if (!Reader->GetFrameInfo(bIsContainerFile ? SequencePath : ImagePaths[0][0], FirstFrameInfo))
		{
			UE_LOGF(LogImgMedia, Error, "Failed to get frame information from first image in %ls", *SequencePath);
			return false;
		}
	}
	
	if (FirstFrameInfo.UncompressedSize == 0)
	{
		UE_LOGF(LogImgMedia, Error, "The first image in sequence %ls does not have a valid frame size", *SequencePath);
		return false;
	}

	if (FirstFrameInfo.Dim.GetMin() <= 0)
	{
		UE_LOGF(LogImgMedia, Error, "The first image in sequence %ls does not have a valid dimension", *SequencePath);
		return false;
	}

	NumMipLevels = FMath::Max(FirstFrameInfo.NumMipLevels, bIsContainerFile ? 1 : ImagePaths.Num());
	NumBytesPerPixel = FirstFrameInfo.NumBytesPerPixel;

	MinimumMipLevelToUpscale = -1;

	SequenceDim = FirstFrameInfo.Dim;

	if (FrameRateOverride.IsValid() && (FrameRateOverride.Numerator > 0))
	{
		FirstFrameInfo.FrameRate = FrameRateOverride;
		SequenceFrameRate = FrameRateOverride;
	}
	else
	{
		SequenceFrameRate = FirstFrameInfo.FrameRate;
	}

	SequenceDuration = FrameNumberToTime(GetNumImages());
	SIZE_T UncompressedSize = FirstFrameInfo.UncompressedSize;

	// With mips in separate folders, we scale the uncompressed size by its mip chain approximation.
	// With mips in each frame, the readers already return a value that incorporates their size.
	if (MipsInSeparateFiles())
	{
		UncompressedSize = (UncompressedSize * 4) / 3;
	}

	if (FirstFrameInfo.bHasTiles)
	{
		TilingDescription.TileBorderSize = FirstFrameInfo.TileBorder;
		TilingDescription.TileNum = FirstFrameInfo.NumTiles;
		TilingDescription.TileSize = FirstFrameInfo.TileDimensions;
	}

#if WITH_EDITOR
	// Get required bandwidth needed for all mips and tiles.
	float FrameTime = SequenceFrameRate.AsInterval();
	if (FrameTime > 0)
	{
		Bandwidth.Required = UncompressedSize / FrameTime;
	}
#endif

	// Push the sequence info into the resolver so it's ready when providers register.
	{
		FMediaSequenceInfo SequenceInfo;
		SequenceInfo.Name = SequenceName;
		SequenceInfo.NumMipLevels = FMath::Max(1, NumMipLevels);
		SequenceInfo.Dim = SequenceDim;
		SequenceInfo.TilingDescription = TilingDescription;
		SequenceInfo.TilingDescription.TileNum.X = FMath::Max(1, SequenceInfo.TilingDescription.TileNum.X);
		SequenceInfo.TilingDescription.TileNum.Y = FMath::Max(1, SequenceInfo.TilingDescription.TileNum.Y);
		TileVisibilityResolver->SetSequenceInfo(SequenceInfo);
	}

	// initialize loader
	const FPlatformMemoryStats Stats = FPlatformMemory::GetStats();

	// We always need to cache at least 2 frames - current and the future frame.
	constexpr int32 MinNumberOfFramesToLoad = 2;

	// Are we using the smart cache?
	if (SmartCacheSettings.bIsEnabled)
	{
		// Load frames that are a certain amount of time ahead of us.
		float TimeToLookAhead = SmartCacheSettings.TimeToLookAhead;

		NumFramesToLoad = TimeToLookAhead / SequenceFrameRate.AsInterval();
		// Clamp min frame to load to 1 so we always try to load something.
		int32 NumImages = GetNumImages();
		int32 MinFrames = FMath::Min(MinNumberOfFramesToLoad, NumImages);
		NumFramesToLoad = FMath::Clamp(NumFramesToLoad, MinFrames, NumImages);
		NumLoadBehind = 0;
		NumLoadAhead = NumFramesToLoad;
	}
	else
	{
		const SIZE_T DesiredCacheSize = Settings->CacheSizeGB * 1024 * 1024 * 1024;
		const SIZE_T CacheSize = FMath::Clamp(DesiredCacheSize, (SIZE_T)0, (SIZE_T)Stats.AvailablePhysical);
		const int32 NumImages = GetNumImages();
		const int32 MaxFramesToLoad = FMath::Min((int32)(CacheSize / UncompressedSize), NumImages);
		const int32 MinFramesToLoad = FMath::Min(MinNumberOfFramesToLoad, NumImages);

		if (Settings->bEnablePrecachingTimeInterval)
		{
			// Desired number of frames specified by a time interval, clamped by the max desired size.
			NumFramesToLoad = FMath::Clamp(Settings->PrecachingTimeInterval / SequenceFrameRate.AsInterval(), MinFramesToLoad, MaxFramesToLoad);
		}
		else
		{
			// Desired number of frames specified by the max desired size.
			NumFramesToLoad = FMath::Max(MaxFramesToLoad, MinFramesToLoad);
		}

		// Apply look ahead/behind ratio.
		const float LoadBehindScale = FMath::Clamp(Settings->CacheBehindPercentage, 0.0f, 100.0f) / 100.0f;
		NumLoadBehind = FMath::TruncToInt32(LoadBehindScale * NumFramesToLoad);
		NumLoadAhead = FMath::CeilToInt32((1.0f - LoadBehindScale) * NumFramesToLoad);
	}

	Frames.Empty(NumFramesToLoad);

	// update info
	Info = TEXT("Image Sequence\n");
	Info += FString::Printf(TEXT("    Dimension: %i x %i\n"), SequenceDim.X, SequenceDim.Y);
	Info += FString::Printf(TEXT("    Format: %s\n"), *FirstFrameInfo.FormatName);
	Info += FString::Printf(TEXT("    Compression: %s\n"), *FirstFrameInfo.CompressionName);
	Info += FString::Printf(TEXT("    Frames: %i\n"), GetNumImages());
	Info += FString::Printf(TEXT("    Frame Rate: %.2f (%i/%i)\n"), SequenceFrameRate.AsDecimal(), SequenceFrameRate.Numerator, SequenceFrameRate.Denominator);

	return true;
}

void FImgMediaLoader::WarmupSequence(const FImgMediaFrameInfo& InFirstFrameInfo, bool Loop, bool bIsSequencerEnvironment)
{
	/**
	* With sequencer, the warmup pre-allocations had negative consequences outweighting any potential benefits:
	* - Prevented asynchronous work in pre-roll section due to spike in render thread.
	* - In cases where only a few mips or tiles are visible, the allocations were overly greedy.
	* - Increased risk of blocking playback disabling if warmup allocations exceed timeout.
	*/
	if (!bIsSequencerEnvironment)
	{
		// Giving our reader a chance to handle RAM allocation.
		// Not all readers use this, only those that need to handle large files
		// or need to be as efficient as possible.
		Reader->PreAllocateMemoryPool(NumFramesToLoad, InFirstFrameInfo);
	}

	FScopeLock Lock(&CriticalSection);

	Update(0, 0.0f, Loop);
}

void FImgMediaLoader::FindFiles(const FString& SequencePath, TArray<FString>& OutputPaths)
{
	// locate image sequence files
	TArray<FString> FoundFiles;
	IFileManager::Get().FindFiles(FoundFiles, *SequencePath, TEXT("*"));

	UE_LOGF(LogImgMedia, Verbose, "Loader %p: Found %i image files in %ls", this, FoundFiles.Num(), *SequencePath);

	// Same list of extensions as FGenericImgMediaReader::GetFrameInfoFromImage() with the addition of exr & dds
	TSet<FString> SupportedImageFileExtensions = { TEXT("exr"), TEXT("jpg"), TEXT("jpeg"), TEXT("png"), TEXT("bmp"), TEXT("dds") };
	SupportedImageFileExtensions.Append(FTmvImgMediaReader::GetSupportedImageFileExtensions());

	TArray<FString> UnnumberedFiles;
	TSortedMap<int32, FString> SortedFoundFiles;
	SortedFoundFiles.Reserve(FoundFiles.Num());

	int32 FrameNumber;
	for (FString& File : FoundFiles)
	{
		const FString FileExtension = FPaths::GetExtension(File).ToLower();

		if (!SupportedImageFileExtensions.Contains(FileExtension))
		{
			continue;
		}

		if (GetNumberAtEndOfString(FrameNumber, File, FileExtension))
		{
			SortedFoundFiles.Add(FrameNumber, File);
		}
		else
		{
			UnnumberedFiles.Add(File);
		}
	}

	int32 LastIndex = -1;
	const int32 MaxSequenceGap = CVarImgMediaMaxFrameSequenceGap.GetValueOnAnyThread();
	for (const auto& Pair : SortedFoundFiles)
	{
		// Can we fill in gaps in the sequence?
		if (bFillGapsInSequence)
		{
			// Get the index of this file.
			int32 ThisIndex = Pair.Key;

			// Fill in any gaps from the last frame.
			if ((LastIndex != -1) && (LastIndex < ThisIndex - 1))
			{
				// Protect against large gaps that could lead to out of memory allocation.
				const int32 SequenceGap = (ThisIndex - 1) - LastIndex;
				if (SequenceGap < MaxSequenceGap)
				{
					for (int32 Index = 0; Index < SequenceGap; ++Index)
					{
						const FString EmptyString;
						OutputPaths.Add(EmptyString);
					}
				}
				else
				{
					UE_LOGF(LogImgMedia, Warning,
						"Loader %p, sequence \"%ls\": Ignoring large frame gap (%d frames) from frame %d to %d." " If this is expected, adjust cvar \"ImgMedia.MaxFrameSequenceGap\" (current: %d) accordingly.",
						this, *SequencePath, SequenceGap, LastIndex, ThisIndex, MaxSequenceGap);
				}
			}

			LastIndex = ThisIndex;
		}

		OutputPaths.Add(FPaths::Combine(SequencePath, Pair.Value));
	}

	// We still want to support unnumbered files, so we append them to the numbered list.
	UnnumberedFiles.Sort();

	for (const FString& UnnumberedFile : UnnumberedFiles)
	{
		OutputPaths.Add(FPaths::Combine(SequencePath, UnnumberedFile));
	}

}

void FImgMediaLoader::FindMips(const FString& SequencePath)
{
	// Remove trailing '/'.
	FString SequenceDir = FPaths::GetPath(SequencePath);

	// Get parent directory.
	FString BaseName = FPaths::GetCleanFilename(SequenceDir);
	FString ParentDir = FPaths::GetPath(SequenceDir);

	// Is this a mipmap level, i.e. something like 256x256?
	int32 Index = 0;
	const bool bFoundDelimeter = SequenceDir.FindLastChar(TEXT('x'), Index);
	if (bFoundDelimeter)
	{
		// Loop over all mip levels.
		FString BaseMipDir = FPaths::GetPath(SequenceDir);
		int32 MipLevelWidth = 0;
		int32 MipLevelHeight = 0;

		// Getting both left and right components of resolution for mips.
		{
			FString MipLevelStringHeight = SequenceDir.RightChop(Index + 1);
			FString MipLevelStringWidth = SequenceDir.LeftChop(SequenceDir.Len() - Index);
			MipLevelHeight = FCString::Atoi(*MipLevelStringHeight);
			int32 IndexLeft = MipLevelStringWidth.Len() - 1;
			for (; IndexLeft > 0; IndexLeft--)
			{
				FString TempChop = MipLevelStringWidth.RightChop(IndexLeft);
				if (!FCString::IsNumeric(*TempChop))
				{
					break;
				}
			}
			MipLevelStringWidth = MipLevelStringWidth.RightChop(IndexLeft + 1);
			MipLevelWidth = FCString::Atoi(*MipLevelStringWidth);
		}

		while (MipLevelWidth > 1 && MipLevelHeight > 1)
		{
			// Next level down.
			MipLevelWidth /= 2;
			MipLevelHeight /= 2;

			// Try and find files for this mip level.
			FString MipDir = FPaths::Combine(BaseMipDir, FString::Printf(TEXT("%dx%d"), MipLevelWidth, MipLevelHeight));
			TArray<FString> MipFiles;
			FindFiles(MipDir, MipFiles);

			// Stop once we don't find any files.
			if (MipFiles.Num() == 0)
			{
				break;
			}

			// Make sure we have the same number of mip files over all levels.
			if (MipFiles.Num() != GetNumImages())
			{
				UE_LOGF(LogImgMedia, Error, "Loader %p: Found %d images, expected %d, for %ls",
					this, MipFiles.Num(), GetNumImages(), *MipDir);
				break;
			}

			// Make sure we don't have too many levels.
			if (ImagePaths.Num() >= MAX_MIPMAP_LEVELS)
			{
				UE_LOGF(LogImgMedia, Error, "Loader %p: Found too many mipmap levels (max:%d) for %ls",
					this, MAX_MIPMAP_LEVELS, *MipDir);
				break;
			}

			// OK add this level to the list.
			ImagePaths.Emplace(MipFiles);
		}
	}

	if (ImagePaths.Num() > 1)
	{
		UE_CALL_ONCE([&] { UE_LOGF(LogImgMedia, Warning, "Loading mips from separate folders will be deprecated soon (%ls). Please prefer using TMV, EXR or DDS with mip levels.", *SequencePath) });
	}
}


uint32 FImgMediaLoader::TimeToFrameNumber(FTimespan Time) const
{
	if ((Time < FTimespan::Zero()) || (Time >= SequenceDuration) || GetNumImages() < 1)
	{
		return INDEX_NONE;
	}

	/*
	* Older versions of this code attempted to help frame selection by using an Epsilon value
	* during frame selection to compensate for numerical issues. We now use frame vs. time range
	* requested coverage analysis to select the frames, which inherently will compensate for most / all
	* cases covered by the older code.
	*/

	return UE::ImgMediaLoaderUtils::TimeToFrameNumber(Time, SequenceFrameRate);
}

FTimespan FImgMediaLoader::FrameNumberToTime(uint32 FrameNumber) const
{
	// Note: this version is reversible with TimeToFrameNumber.
	return UE::ImgMediaLoaderUtils::GetFrameStartTime(FrameNumber, SequenceFrameRate);
}

int64 FImgMediaLoader::TimeToFrameNumberUnbound(FTimespan Time) const
{
	return UE::ImgMediaLoaderUtils::TimeToFrameNumberUnbound(Time, SequenceFrameRate);
}


void FImgMediaLoader::Update(int32 PlayHeadFrame, float PlayRate, bool Loop)
{
	// In case reader needs to do something once per frame.
	// As an example return buffers back to the pool in ExrImgMediaReaderGpu.
	Reader->OnTick();

	// @todo gmp: ImgMedia: take PlayRate and DeltaTime into account when determining frames to load
	const int32 NumImagePaths = GetNumImages();

	// determine frame numbers to be loaded
	TArray<int32> FramesToLoad;
	{
		FramesToLoad.Empty(NumFramesToLoad);

		int32 FrameOffset = (PlayRate >= 0.0f) ? 1 : -1;

		int32 LoadAheadCount = NumLoadAhead;
		int32 LoadAheadIndex = PlayHeadFrame;

		int32 LoadBehindCount = NumLoadBehind;
		int32 LoadBehindIndex = PlayHeadFrame - FrameOffset;

		// Adjust if we are skipping frames.
		if (SkipFramesLevel > 0)
		{
			int32 Mult = (1 << SkipFramesLevel);
			LoadAheadIndex = GetSkipFrame(LoadAheadIndex);
			LoadBehindIndex = GetSkipFrame(LoadBehindIndex);
			FrameOffset *= Mult;
		}

		// alternate between look ahead and look behind

		while ((FramesToLoad.Num() < NumFramesToLoad) &&
			((LoadAheadCount > 0) || (LoadBehindCount > 0)))
		{
			if (LoadAheadCount > 0)
			{
				if (LoadAheadIndex < 0)
				{
					if (Loop)
					{
						LoadAheadIndex += NumImagePaths;
						LoadAheadIndex = GetSkipFrame(LoadAheadIndex);
					}
					else
					{
						LoadAheadCount = 0;
					}
				}
				else if (LoadAheadIndex >= NumImagePaths)
				{
					if (Loop)
					{
						LoadAheadIndex -= NumImagePaths;
						LoadAheadIndex = GetSkipFrame(LoadAheadIndex);
					}
					else
					{
						LoadAheadCount = 0;
					}
				}

				if (LoadAheadCount > 0)
				{
					FramesToLoad.Add(LoadAheadIndex);
				}

				LoadAheadIndex += FrameOffset;
				--LoadAheadCount;
			}

			if (LoadBehindCount > 0)
			{
				if (LoadBehindIndex < 0)
				{
					if (Loop)
					{
						LoadBehindIndex += NumImagePaths;
						LoadBehindIndex = GetSkipFrame(LoadBehindIndex);
					}
					else
					{
						LoadBehindCount = 0;
					}
				}
				else if (LoadBehindIndex >= NumImagePaths)
				{
					if (Loop)
					{
						LoadBehindIndex -= NumImagePaths;
						LoadBehindIndex = GetSkipFrame(LoadBehindIndex);
					}
					else
					{
						LoadBehindCount = 0;
					}
				}

				if (LoadBehindCount > 0)
				{
					FramesToLoad.Add(LoadBehindIndex);
				}

				LoadBehindIndex -= FrameOffset;
				--LoadBehindCount;
			}
		}
	}

	FScopeLock ScopeLock(&CriticalSection);

	// determine queued frame numbers that can be discarded
	for (int32 Idx = QueuedFrameNumbers.Num() - 1; Idx >= 0; --Idx)
	{
		const int32 FrameNumber = QueuedFrameNumbers[Idx].Frame;

		if (!FramesToLoad.Contains(FrameNumber))
		{
			UE_LOGF(LogImgMedia, Verbose, "Loader %p: Removed Frame %i", this, FrameNumber);
			QueuedFrameNumbers.RemoveAtSwap(Idx);
			Reader->CancelFrame(FrameNumber);

			// We failed to load this frame... are we throttling?
			if (bIsSkipFramesEnabled)
			{
				SkipFramesCounter++;
				if (SkipFramesCounter > SkipFramesCounterRaiseLevelThreshold)
				{
					SkipFramesLevel++;
					SkipFramesCounter = SkipFramesCounterNewLevelValue;
				}
			}
		}
	}

	// determine frame numbers that need to be cached
	PendingFrameNumbers.Empty();

	// Limit which frames can be invalidated for missing mips/tiles
	const int32 FrameInvalidationMaxCount = CVarImgMediaFrameInvalidationMaxCount.GetValueOnAnyThread();

	for (int32 FrameNumber : FramesToLoad)
	{
		// Get frame from cache.
		bool NeedFrame = false;
		const TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>* FramePtr = GetCachedFrame(FrameNumber);

		// Did we get a frame?
		if ((FramePtr == nullptr) || ((*FramePtr).IsValid() == false))
		{
			// No, we need one.
			NeedFrame = true;
		}
		else if(ImgMediaLoader::IsCachedFrameInRange(FrameNumber, PlayHeadFrame, FrameInvalidationMaxCount, NumImagePaths, PlayRate, bIsPaused))
		{
			// Yes. Check if we have all the desired tiles per mip level.
			TMap<int32, FMediaTileSelection> DesiredMipsAndTiles;
			GetDesiredMipTiles(FrameNumber, DesiredMipsAndTiles);

			NeedFrame = !(*FramePtr)->MipTilesPresent.ContainsMipTiles(DesiredMipsAndTiles);

			if (NeedFrame)
			{
				UE_LOGF(LogImgMedia, VeryVerbose, "Loader %p [%ls]: Invalidating frame %i at play head %i due to missing mips or tiles.", this, *PlayerGuid.ToString(), FrameNumber, PlayHeadFrame);
			}
		}

		if (NeedFrame)
		{
			bool FoundFrame = false;
			for (const auto& Entry : QueuedFrameNumbers)
			{
				if (Entry.Frame == FrameNumber)
				{
					FoundFrame = true;
					break;
				}
			}

			if (!FoundFrame)
			{
				// Do we actually have a frame for this?
				// For file sequences, check if the path is empty (gap-filled placeholder).
				// Containers always have valid frames for all sample indices.
				if (!bIsContainerFile && ImagePaths[0][FrameNumber].Len() == 0)
				{
					TryAddEmptyFrame(FrameNumber);
				}
				else
				{
					PendingFrameNumbers.Add(FrameNumber);
				}
			}
		}
	}
	Algo::Reverse(PendingFrameNumbers);

	// Are we throttling and there no frames pending?
	if ((bIsSkipFramesEnabled) && (PendingFrameNumbers.Num() == 0))
	{
		// This implies that we have spare bandwidth, so lower the skip count.
		if (SkipFramesCounter > 0)
		{
			SkipFramesCounter--;
			if (SkipFramesCounter <= 0)
			{
				if (SkipFramesLevel > 0)
				{
					SkipFramesLevel--;
					SkipFramesCounter = SkipFramesCounterNewLevelValue;
				}
			}
		}
	}
	CSV_EVENT(ImgMedia, TEXT("LoaderUpdatePending %d %d"), (PendingFrameNumbers.Num() > 0) ? PendingFrameNumbers[0] : -1, (PendingFrameNumbers.Num() > 0) ? PendingFrameNumbers[PendingFrameNumbers.Num() - 1] : -1);
}


/* IImgMediaLoader interface
 *****************************************************************************/


bool FImgMediaLoader::TryAddEmptyFrame(int32 FrameNumber)
{
	FScopeLock Lock(&CriticalSection);

	bool bAddFrame = UseGlobalCache ? !GlobalCache->Contains(SequenceName, FrameNumber) : !Frames.Contains(FrameNumber);

	if (bAddFrame)
	{
		const int32 NumChannels = 3;
		const int32 PixelSize = sizeof(uint16) * NumChannels;
		TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> Frame = MakeShared<FImgMediaFrame>();

		FImgMediaFrameInfo FrameInfo = {};
		FrameInfo.Dim = SequenceDim;
		FrameInfo.FrameRate = SequenceFrameRate;
		FrameInfo.NumChannels = NumChannels;
		FrameInfo.bHasTiles = false;
		Frame->SetInfo(FrameInfo);

		Frame->Format = EMediaTextureSampleFormat::FloatRGB;
		Frame->Stride = SequenceDim.X * PixelSize;

		AddFrameToCache(FrameNumber, Frame);
	}

	return bAddFrame;
}

void FImgMediaLoader::NotifyWorkComplete(FImgMediaLoaderWork& CompletedWork, int32 JobID, int32 FrameNumber,
	const TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>& Frame, float WorkTime)
{
	bool bFrameWasAdded = false;
	bool bFrameMatchesLastRequested = false;
	{
		FScopeLock Lock(&CriticalSection);

		// if frame is still needed, add it to the cache
		if (QueuedFrameNumbers.Remove({ FrameNumber, JobID }) > 0)
		{
			AddFrameToCache(FrameNumber, Frame);
			bFrameWasAdded = true;

			// Reduce SkipFramesCounter when at skip level 0, otherwise it will never go down
			// unless we have a lot more bandwidth than needed.
			if ((bIsSkipFramesEnabled) && (SkipFramesLevel == 0) && (SkipFramesCounter > 0))
			{
				SkipFramesCounter--;
			}
		}

		// Make sure the frame is no longer canceled, otherwise we might end up cancelling this frame
		// later on inadvertently!
		Reader->UncancelFrame(FrameNumber);

		WorkPool.Push(&CompletedWork);

#if WITH_EDITOR
		// Update bandwidth. (Keep inside CriticalSection)
		if (Frame.IsValid() && WorkTime > 0.0)
		{
			Bandwidth.Update(Frame, WorkTime);
		}
#endif

		// Captured under the lock - LastRequestedFrame is written from the game thread without
		// synchronization with this worker callback, so the comparison must happen here.
		bFrameMatchesLastRequested = (FrameNumber == LastRequestedFrame);
	}

	// Tiles for the displayed frame just changed; signal the facade to drop its sample cache.
	// Outside CriticalSection because ReceiveMediaEvent takes its own locks on the facade side.
	// Note: a 1-frame sequence is functionally paused, so it will go through the paused refresh path.
	const bool bShouldRefresh = bIsPaused.load(std::memory_order_relaxed) || GetNumImages() == 1;
	if (bShouldRefresh && bFrameWasAdded && bFrameMatchesLastRequested
		&& EventSink != nullptr
		&& CVarImgMediaPausedRefreshEnabled.GetValueOnAnyThread())
	{
		EventSink->ReceiveMediaEvent(EMediaEvent::Internal_RequestVideoSampleRefresh);
	}
}

void FImgMediaLoader::AddFrameToCache(int32 FrameNumber, const TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>& Frame)
{
	if (Frame.IsValid())
	{
		UE_LOGF(LogImgMedia, VeryVerbose, "Loader %p: Loaded frame %i", this, FrameNumber);
		if (UseGlobalCache)
		{
			if (GlobalCache->FindAndTouch(SequenceName, FrameNumber) == nullptr)
			{
				const FString& FrameKey = bIsContainerFile ? SequenceName.ToString() : ImagePaths[0][FrameNumber];
				GlobalCache->AddFrame(FrameKey, SequenceName, FrameNumber, Frame, MipsInSeparateFiles());
			}
		}
		else
		{
			Frames.Add(FrameNumber, Frame);
		}
	}
}

void FImgMediaLoader::GetDesiredMipTiles(int32 FrameIndex, TMap<int32, FMediaTileSelection>& OutMipsAndTiles) const
{
	// note: While currently unused, FrameIndex could become useful for recorded frames.

	if (HasMipsOrTilesEnabled())
	{
		if (const TSharedPtr<FMediaTileVisibilityRequest, ESPMode::ThreadSafe> Snapshot = TileVisibilityResolver->GetSnapshot())
		{
			OutMipsAndTiles = Snapshot->VisibleTiles;
			return;
		}
	}

	//Fallback case where we activate all mips and tiles.
	for (int32 MipLevel = 0; MipLevel < GetNumMipLevels(); ++MipLevel)
	{
		OutMipsAndTiles.Emplace(MipLevel, FMediaTileSelection::CreateForTargetMipLevel(SequenceDim, TilingDescription.TileSize, MipLevel, true));
	}
}

int32 FImgMediaLoader::GetDesiredMinimumMipLevelToUpscale() const
{
	if (HasMipsOrTilesEnabled())
	{
		return TileVisibilityResolver->GetMinimumMipLevelToUpscale();
	}

	return -1;
}


FTimespan FImgMediaLoader::ModuloTime(FTimespan Time) const
{
	bool IsNegative = Time < FTimespan::Zero();

	FTimespan NewTime = Time % SequenceDuration;
	if (IsNegative)
	{
		NewTime = SequenceDuration + NewTime;
	}

	return NewTime;
}


bool FImgMediaLoader::GetNumberAtEndOfString(int32 &Number, const FString& String, const FString& Extension) const
{
	bool bFoundNumber = false;

	// Find the first digit starting from the right, excluding the extension (in case it has a number).
	int32 Index = String.Len() - 1 - Extension.Len();
	for (; Index >= 0; --Index)
	{
		const bool bIsDigit = FChar::IsDigit(String[Index]);
		if (bIsDigit)
		{
			break;
		}
	}

	// Did we find one?
	if (Index >= 0)
	{
		int32 LastNumberIndex = Index;
		// Find the next non digit...
		for (; Index >= 0; --Index)
		{
			bool bIsDigit = FChar::IsDigit(String[Index]);
			if (bIsDigit == false)
			{
				break;
			}
		}

		// Index now points to the next non digit.
		// Extract the number.
		Index++;
		if (Index < String.Len())
		{
			bFoundNumber = true;
			FString NumberString = String.Mid(Index, LastNumberIndex - Index + 1);
			Number = FCString::Atoi(*NumberString);
		}
	}

	return bFoundNumber;
}


int32 FImgMediaLoader::GetSkipFrame(int32 FrameIndex) const
{
	int32 Mult = (1 << SkipFramesLevel);
	int32 Mask = ~(Mult - 1);
	FrameIndex = (FrameIndex & Mask) + (Mult - 1);

	const int32 NumImagePaths = GetNumImages();
	if (FrameIndex >= NumImagePaths)
	{
		FrameIndex = NumImagePaths - 1;
	}

	return FrameIndex;
}

void FImgMediaLoader::UpdateBandwidthThrottling()
{
	bIsSkipFramesEnabled = bIsBandwidthThrottlingEnabled && (bIsPlaybackBlocking == false);
	if (bIsSkipFramesEnabled == false)
	{
		SkipFramesCounter = 0;
		SkipFramesLevel = 0;
	}
}

const TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>* FImgMediaLoader::GetCachedFrame(int32 InFrameNumber)
{
	if (UseGlobalCache)
	{
		return GlobalCache->FindAndTouch(SequenceName, InFrameNumber);
	}
	else
	{
		return Frames.FindAndTouch(InFrameNumber);
	}
}

