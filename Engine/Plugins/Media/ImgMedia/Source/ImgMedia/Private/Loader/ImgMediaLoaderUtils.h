// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "Misc/Timespan.h"

/**
 * Frame conversion math utilities.
 * Extracted from FImgMediaLoader for unit tests.
 */
namespace UE::ImgMediaLoaderUtils
{
	/**
	 * Converts the given time to a frame number at the given framerate.
	 * @param InTime Time (timespan in ticks)
	 * @param InFrameRate Frame rate in frames per second
	 * @return Frame number
	 */
	inline uint32 TimeToFrameNumber(const FTimespan& InTime, const FFrameRate& InFrameRate)
	{
		return static_cast<uint32>(FMath::DivideAndRoundDown(InTime.GetTicks() * InFrameRate.Numerator, InFrameRate.Denominator * ETimespan::TicksPerSecond));
	}

	/**
	 * Converts the given time to a frame number at the given framerate.
	 * @param InTime Time (timespan in ticks)
	 * @param InFrameRate Frame rate in frames per second
	 * @return Frame number (can be very large)
	 */
	inline int64 TimeToFrameNumberUnbound(const FTimespan& InTime, const FFrameRate& InFrameRate)
	{
		return FMath::DivideAndRoundDown(InTime.GetTicks() * InFrameRate.Numerator, InFrameRate.Denominator * ETimespan::TicksPerSecond);
	}

	/**
	 * Returns the lower bound, start time, of the given frame number for the given frame rate.
	 * 
	 * @param InFrameNumber Frame number
	 * @param InFrameRate Frame rate in frames per second
	 * @return A timespan in tick, where the tick number is the first tick that will return back the same frame number from TimeToFrameNumber.
	 */
	inline FTimespan GetFrameStartTime(uint32 InFrameNumber, const FFrameRate& InFrameRate)
	{
		const int64 DenominatorTicks = InFrameRate.Denominator * ETimespan::TicksPerSecond;
		const FTimespan FloorTicks(FMath::DivideAndRoundDown(int64(InFrameNumber) * DenominatorTicks, int64(InFrameRate.Numerator)));

		// Test the case where floor of tick will be reversible, in that case, it is the first tick of that frame.
		if (TimeToFrameNumber(FloorTicks, InFrameRate) == InFrameNumber)
		{
			return FloorTicks;
		}

		// Ceil of Ticks is always reversible with TimeToFrameNumber, but is not always the first tick of the frame.
		return FTimespan(FMath::DivideAndRoundUp(int64(InFrameNumber) * DenominatorTicks, int64(InFrameRate.Numerator)));
	}
}