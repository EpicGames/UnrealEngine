// Copyright Epic Games, Inc. All Rights Reserved.

#include "Loader/ImgMediaLoaderWork.h"
#include "ImgMediaPrivate.h"

#include "Misc/ScopeLock.h"
#include "IImgMediaModule.h"
#include "Readers/IImgMediaReader.h"
#include "Loader/ImgMediaLoader.h"


/** Time spent abandoning worker threads. */
DECLARE_CYCLE_STAT(TEXT("ImgMedia Loader Abadon Work"), STAT_ImgMedia_LoaderAbandonWork, STATGROUP_Media);

/** Time spent finalizing worker threads. */
DECLARE_CYCLE_STAT(TEXT("ImgMedia Loader Finalize Work"), STAT_ImgMedia_LoaderFinalizeWork, STATGROUP_Media);

/** Time spent reading image frames in worker threads. */
DECLARE_CYCLE_STAT(TEXT("ImgMedia Loader Read Frame"), STAT_ImgMedia_LoaderReadFrame, STATGROUP_Media);


/* FImgMediaLoaderWork structors
 *****************************************************************************/

FImgMediaLoaderWork::FImgMediaLoaderWork(const TSharedRef<FImgMediaLoader, ESPMode::ThreadSafe>& InOwner, const TSharedRef<IImgMediaReader, ESPMode::ThreadSafe>& InReader)
	: FrameNumber(INDEX_NONE)
	, OwnerPtr(InOwner)
	, Reader(InReader)
{ }


/* FImgMediaLoaderWork interface
 *****************************************************************************/

void FImgMediaLoaderWork::Initialize(int32 InJobID, int32 InFrameNumber,
	TMap<int32, FMediaTileSelection> InMipTiles,
	TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> InExistingFrame)
{
	JobID = InJobID;
	FrameNumber = InFrameNumber;
	MipTiles = MoveTemp(InMipTiles);
	ExistingFrame = InExistingFrame;

	// Ensure that we start from mip0 when iterating the map for consistency
	MipTiles.KeySort(TLess<int32>());
}


/* IQueuedWork interface
 *****************************************************************************/

void FImgMediaLoaderWork::Abandon()
{
	SCOPE_CYCLE_COUNTER(STAT_ImgMedia_LoaderAbandonWork);
	CSV_EVENT(ImgMedia, TEXT("LoaderAbandon %d"), FrameNumber);

	Finalize(nullptr, 0.0f);
}


void FImgMediaLoaderWork::DoThreadedWork()
{
	TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> Frame;
	float WorkTime = 0.0f;
	UE_LOGF(LogImgMedia, Verbose, "Loader %p: Starting to read %i", this, FrameNumber);

	if (FrameNumber == INDEX_NONE)
	{
		Frame.Reset();
	}
	else
	{
		SCOPE_CYCLE_COUNTER(STAT_ImgMedia_LoaderReadFrame);
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("ImgMedia_LoaderReadFrame %d"), FrameNumber));

		// read the image frame
		if (ExistingFrame == nullptr)
		{
			Frame = MakeShareable(new FImgMediaFrame());
		}
		else
		{
			Frame = ExistingFrame;
			Frame->NumTilesRead = 0;
		}

		double StartTime = FPlatformTime::Seconds();
		if(!Reader->ReadFrame(FrameNumber, MipTiles, Frame))
		{
			if (ExistingFrame == nullptr)
			{
				Frame.Reset();
			}
		}
		else
		{
			WorkTime = FPlatformTime::Seconds() - StartTime;
		}
	}

	SCOPE_CYCLE_COUNTER(STAT_ImgMedia_LoaderFinalizeWork);

	Finalize(Frame, WorkTime);
}


/* FImgMediaLoaderWork implementation
 *****************************************************************************/

void FImgMediaLoaderWork::Finalize(TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> Frame, float WorkTime)
{
	ExistingFrame.Reset();

	TSharedPtr<FImgMediaLoader, ESPMode::ThreadSafe> Owner = OwnerPtr.Pin();

	if (Owner.IsValid())
	{
		Owner->NotifyWorkComplete(*this, JobID, FrameNumber, Frame, WorkTime);
	}
	else
	{
		Frame.Reset();
		Frame = nullptr;
		delete this; // owner is gone, self destruct!
	}
}
