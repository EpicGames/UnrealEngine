// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Assets/MediaTileSelection.h"
#include "Misc/IQueuedWork.h"

class FImgMediaLoader;
class IImgMediaReader;

struct FImgMediaFrame;


/**
 * Loads a single image frame from disk.
 */
class FImgMediaLoaderWork
	: public IQueuedWork
{
public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InOwner The loader that created this work item.
	 * @param InReader The image reader to use.
	 */
	FImgMediaLoaderWork(const TSharedRef<FImgMediaLoader, ESPMode::ThreadSafe>& InOwner, const TSharedRef<IImgMediaReader, ESPMode::ThreadSafe>& InReader);

public:

	/**
	 * Initialize this work item.
	 *
	 * @param InJobID An ID to identify the job while in flight
	 * @param InFrameNumber The number of the image frame.
	 * @param InMipLevel Will read in this level and all higher levels.
	 * @param InTileSelection Which tiles to read.
	 * @param InExistingFrame If set, then use this frame to load into.
	 * @see Shutdown
	 */
	void Initialize(int32 InJobID, int32 InFrameNumber, TMap<int32, FMediaTileSelection> InMipTiles, TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> InExistingFrame);

public:

	//~ IQueuedWork interface

	virtual void Abandon() override;
	virtual void DoThreadedWork() override;

protected:

	/**
	 * Notify the owner of this work item, or self destruct.
	 *
	 * @param Frame The frame that was loaded by this work item.
	 * @param WorkTime How long to read this frame (in seconds).
	 */
	void Finalize(TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> Frame, float WorkTime);

private:

	/** ID to identify this workload instance when in flight */
	int32 JobID;

	/** The number of the image frame. */
	int32 FrameNumber;

	/** Which tiles we should load per mip level. */
	TMap<int32, FMediaTileSelection> MipTiles;
	
	/** If set, then load the image into this frame instead of making a new one. */
	TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> ExistingFrame;

	/** The loader that created this reader task. */
	TWeakPtr<FImgMediaLoader, ESPMode::ThreadSafe> OwnerPtr;

	/** The image sequence reader to use. */
	TSharedPtr<IImgMediaReader, ESPMode::ThreadSafe> Reader;
};
