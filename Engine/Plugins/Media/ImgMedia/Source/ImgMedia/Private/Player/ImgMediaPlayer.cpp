// Copyright Epic Games, Inc. All Rights Reserved.

#include "Player/ImgMediaPlayer.h"

#include "Assets/MediaTileVisibility.h"
#include "Assets/Providers/ImgMediaPlaneVisibilityProvider.h"
#include "Assets/Providers/ImgMediaSphereVisibilityProvider.h"
#include "Async/Async.h"
#include "Components/MeshComponent.h"
#include "Decoder/ITmvMediaDemuxerFactory.h"
#include "GameFramework/Actor.h"
#include "IImgMediaModule.h"
#include "IMediaEventSink.h"
#include "IMediaOptions.h"
#include "ITmvMediaModule.h"
#include "ImgMediaPrivate.h"
#include "ImgMediaSettings.h"
#include "ImgMediaSource.h"
#include "Loader/ImgMediaLoader.h"
#include "MediaPlayer.h"
#include "MediaPlayerFacade.h"
#include "MediaTexture.h"
#include "MediaTextureTracker.h"
#include "Misc/Paths.h"
#include "Player/ImgMediaTextureSample.h"
#include "Scheduler/ImgMediaScheduler.h"
#include "UObject/WeakObjectPtr.h"

#if WITH_EDITOR
#include "EngineAnalytics.h"
#endif

#define LOCTEXT_NAMESPACE "FImgMediaPlayer"


/** Time spent closing image media players. */
DECLARE_CYCLE_STAT(TEXT("ImgMedia Player Close"), STAT_ImgMedia_PlayerClose, STATGROUP_Media);

/** Time spent in image media player input tick. */
DECLARE_CYCLE_STAT(TEXT("ImgMedia Player TickInput"), STAT_ImgMedia_PlayerTickInput, STATGROUP_Media);


const FTimespan HackDeltaTimeOffset(1);

#if WITH_EDITOR
namespace
{
	// File-static so the deferred-completion lambda can record analytics from a worker thread
	// without needing to capture FImgMediaPlayer* (which may have been destroyed by then).
	void RecordImgMediaSourceOpenedEvent(const FImgMediaLoader& Loader)
	{
		if (!FEngineAnalytics::IsAvailable())
		{
			return;
		}

		TArray<FAnalyticsEventAttribute> EventAttributes;

		const FIntPoint& Resolution = Loader.GetSequenceDim();
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ResolutionWidth"), FString::Printf(TEXT("%d"), Resolution.X)));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ResolutionHeight"), FString::Printf(TEXT("%d"), Resolution.Y)));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("FrameRate"), Loader.GetSequenceFrameRate().ToPrettyText().ToString()));

		FString FileExtension = FPaths::GetExtension(Loader.GetImagePath(0, 0));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Format"), FileExtension));

		const FString& LayerName = Loader.GetLayerName();
		if (!LayerName.IsEmpty())
		{
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("EXRLayerName"), LayerName));
		}

		TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> ColorSettings = Loader.GetSourceColorSettings();
		if (ColorSettings.IsValid() && ColorSettings->HasColorSpaceOverride())
		{
			const UE::Color::FColorSpace& ColorSpace = ColorSettings->GetColorSpaceOverride(UE::Color::FColorSpace::GetWorking());
			FVector2d Red, Green, Blue, White;
			ColorSpace.GetChromaticities(Red, Green, Blue, White);
			const FString ColorSpaceStr = FString::Format(TEXT("R:{0} G:{1} B:{2} W:{3}"), { Red.ToString(), Green.ToString(), Blue.ToString(), White.ToString() });
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ColorSpaceOverride"), ColorSpaceStr));
		}

		if (ColorSettings.IsValid() && ColorSettings->HasEncodingOverride())
		{
			UE::Color::EEncoding Encoding = ColorSettings->GetEncodingOverride();
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Encoding"), UEnum::GetValueAsString((ETextureSourceEncoding)Encoding)));
		}

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("MediaFramework.ImgMediaSourceOpened"), EventAttributes);
	}
}
#endif


/* FImgMediaPlayer structors
 *****************************************************************************/

FImgMediaPlayer::FImgMediaPlayer(IMediaEventSink& InEventSink, const TSharedRef<FImgMediaScheduler, ESPMode::ThreadSafe>& InScheduler,
	const TSharedRef<FImgMediaGlobalCache, ESPMode::ThreadSafe>& InGlobalCache)
	: CurrentDuration(FTimespan::Zero())
	, CurrentRate(0.0f)
	, LastNonZeroRate(0.0f)
	, CurrentState(EMediaState::Closed)
	, CurrentTime(FTimespan::Zero())
	, CurrentSeekIndex(0)
	, DeltaTimeHackApplied(false)
	, EventSink(InEventSink)
	, LastFetchTime(FTimespan::MinValue())
	, PlaybackRestarted(false)
	, Scheduler(InScheduler)
	, SelectedVideoTrack(INDEX_NONE)
	, ShouldLoop(false)
	, GlobalCache(InGlobalCache)
	, RequestFrameHasRun(true)
	, PlaybackIsBlocking(false)
{ }


FImgMediaPlayer::~FImgMediaPlayer()
{
	Close(); // class is final
	RemoveMediaTextureTrackerEventHandlers();
}

/* IMediaPlayer interface
 *****************************************************************************/

void FImgMediaPlayer::Close()
{
	SCOPE_CYCLE_COUNTER(STAT_ImgMedia_PlayerClose);

	if (!Loader.IsValid())
	{
		return;
	}

	RemoveMediaTextureTrackerEventHandlers();
	MediaTextureWeakPtr.Reset();
	OwnedTileVisibilityProviders.Reset();

	Scheduler->UnregisterLoader(Loader.ToSharedRef());
	Loader.Reset();

	CurrentDuration = FTimespan::Zero();
	CurrentUrl.Empty();
	CurrentRate = 0.0f;
	LastNonZeroRate = 0.0f;
	CurrentState = EMediaState::Closed;
	CurrentTime = FTimespan::Zero();
	CurrentSeekIndex = 0;
	DeltaTimeHackApplied = false;
	LastFetchTime = FTimespan::MinValue();
	PlaybackRestarted = false;
	SelectedVideoTrack = INDEX_NONE;
	PlaybackIsBlocking = false;

	EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);
	EventSink.ReceiveMediaEvent(EMediaEvent::MediaClosed);
}


IMediaCache& FImgMediaPlayer::GetCache()
{
	return *this;
}


IMediaControls& FImgMediaPlayer::GetControls()
{
	return *this;
}


FString FImgMediaPlayer::GetInfo() const
{
	return Loader.IsValid() ? Loader->GetInfo() : FString();
}


FVariant FImgMediaPlayer::GetMediaInfo(FName InfoName) const
{
	FVariant Variant;

	// Source num mips?
	if (InfoName == UMediaPlayer::MediaInfoNameSourceNumMips.Resolve())
	{
		if (Loader.IsValid())
		{
			int32 NumMips = Loader->GetNumMipLevels();
			Variant = NumMips;
		}
	}
	// Source num tiles?
	else if (InfoName == UMediaPlayer::MediaInfoNameSourceNumTiles.Resolve())
	{
		if (Loader.IsValid())
		{
			FIntPoint TileNum(Loader->GetNumTilesX(), Loader->GetNumTilesY());
			Variant = TileNum;
		}
	}
	// Delegate remaining queries to the reader (e.g., container metadata like timecode).
	else if (Loader.IsValid() && Loader->IsReaderValid())
	{
		Variant = Loader->GetReader()->GetMediaInfo(InfoName);
	}

	return Variant;
}


FGuid FImgMediaPlayer::GetPlayerPluginGUID() const
{
	static FGuid PlayerPluginGUID(0x0e4a60c0, 0x2c5947ea, 0xb233562a, 0x57e5761c);
	return PlayerPluginGUID;
}


IMediaSamples& FImgMediaPlayer::GetSamples()
{
	return *this;
}


FString FImgMediaPlayer::GetStats() const
{
	FString StatsString;
	{
		StatsString += TEXT("not implemented yet");
		StatsString += TEXT("\n");
	}

	return StatsString;
}


IMediaTracks& FImgMediaPlayer::GetTracks()
{
	return *this;
}


FString FImgMediaPlayer::GetUrl() const
{
	return CurrentUrl;
}


IMediaView& FImgMediaPlayer::GetView()
{
	return *this;
}


bool FImgMediaPlayer::Open(const FString& Url, const IMediaOptions* Options)
{
	UE_LOGF(LogImgMedia, Log, "Open() called without player options, smart caching will not be available.");
	FMediaPlayerOptions NoOptions;
	NoOptions.SetAllAsOptional();
	return Open(Url, Options, &NoOptions);
}

bool FImgMediaPlayer::Open(const FString& Url, const IMediaOptions* Options, const FMediaPlayerOptions* PlayerOptions)
{
	Close();

	FString UrlScheme;
	FString UrlPath;

	if (Url.IsEmpty() || !Url.Split(TEXT("://"), &UrlScheme, &UrlPath))
	{
		return false;
	}
	
	bool bIsSupportedContainer = false;
	if (UrlScheme == TEXT("file"))
	{
		const FString Extension = FPaths::GetExtension(UrlPath);
		if (const ITmvMediaModule* TmvModule = ITmvMediaModule::Get())
		{
			if (TmvModule->FindDemuxerFactoryForExtension(Extension).IsValid())
			{
				bIsSupportedContainer = true;
			}
		}
	}
	
	if (!Url.StartsWith(TEXT("img://")) && !bIsSupportedContainer)
	{
		return false;
	}

	CurrentState = EMediaState::Preparing;
	CurrentUrl = Url;

	// determine image sequence proxy, if any
	FString Proxy;

	if (Options != nullptr)
	{
		Proxy = Options->GetMediaOption(ImgMedia::ProxyOverrideOption, FString());
	}

	if (Proxy.IsEmpty())
	{
		Proxy = GetDefault<UImgMediaSettings>()->GetDefaultProxy();
	}

	auto GetBoolOption = [](const FMediaPlayerOptions* InPlayerOptions, const FName& InOptionName, bool bInDefaultValue) -> bool
	{
		const FVariant* Var = InPlayerOptions ? InPlayerOptions->InternalCustomOptions.Find(InOptionName) : nullptr;
		return Var && Var->GetType() == EVariantTypes::Bool ? Var->GetValue<bool>() : bInDefaultValue;
	};
	auto GetDoubleOption = [](const FMediaPlayerOptions* InPlayerOptions, const FName& InOptionName, double InDefaultValue) -> double
	{
		const FVariant* Var = InPlayerOptions ? InPlayerOptions->InternalCustomOptions.Find(InOptionName) : nullptr;
		return Var && Var->GetType() == EVariantTypes::Float ? (double)Var->GetValue<float>() : Var && Var->GetType() == EVariantTypes::Double ? Var->GetValue<double>() : InDefaultValue;
	};
	auto GetStringOption = [](const FMediaPlayerOptions* InPlayerOptions, const FName& InOptionName) -> FString
	{
		const FVariant* Var = InPlayerOptions ? InPlayerOptions->InternalCustomOptions.Find(InOptionName) : nullptr;
		return Var && Var->GetType() == EVariantTypes::String ? Var->GetValue<FString>() : FString();
	};
	auto GetNameOption = [](const FMediaPlayerOptions* InPlayerOptions, const FName& InOptionName) -> FName
		{
			const FVariant* Var = InPlayerOptions ? InPlayerOptions->InternalCustomOptions.Find(InOptionName) : nullptr;
			return Var && Var->GetType() == EVariantTypes::Name ? Var->GetValue<FName>() : FName();
		};

	// get frame rate override, if any
	FFrameRate FrameRateOverride(0, 0);
	TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> SourceColorSettings;
	bool bFillGapsInSequence = true;
	FString LayerName;
	bool bIsSmartCacheEnabled = GetBoolOption(PlayerOptions, MediaPlayerOptionValues::ImgMediaSmartCacheEnabled(), false);
	bool bIsSequencerEnvironment = GetNameOption(PlayerOptions, MediaPlayerOptionValues::Environment()).IsEqual(MediaPlayerOptionValues::Environment_Sequencer());
	double SmartCacheTimeToLookAhead = GetDoubleOption(PlayerOptions, MediaPlayerOptionValues::ImgMediaSmartCacheTimeToLookAhead(), 0.0);

	UMediaTexture* ViewTexture = nullptr;
	{
		FSoftObjectPath ViewTexturePath = GetStringOption(PlayerOptions, MediaPlayerOptionValues::ViewMediaTexture());
		ViewTexture =  Cast<UMediaTexture>(ViewTexturePath.ResolveObject());
	}

	// Resolve the active media texture for this player so the tile-visibility resolver
	// can discover tracker objects bound to it.
	if (ViewTexture != nullptr)
	{
		MediaTextureWeakPtr = ViewTexture;
	}
	else
	{
		for (TWeakObjectPtr<UMediaTexture> TexturePtr : FMediaTextureTracker::Get().GetTextures())
		{
			UMediaTexture* Texture = TexturePtr.Get();
			if (Texture == nullptr)
			{
				continue;
			}
			UMediaPlayer* MediaPlayer = Texture->GetMediaPlayer();
			if (MediaPlayer == nullptr)
			{
				continue;
			}
			TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> Player = MediaPlayer->GetPlayerFacade()->GetPlayer();
			if (Player.IsValid() && Player.Get() == this)
			{
				MediaTextureWeakPtr = TexturePtr;
				break;
			}
		}
	}

	if (Options != nullptr)
	{
		FrameRateOverride.Denominator = Options->GetMediaOption(ImgMedia::FrameRateOverrideDenonimatorOption, 0LL);
		FrameRateOverride.Numerator = Options->GetMediaOption(ImgMedia::FrameRateOverrideNumeratorOption, 0LL);
		bFillGapsInSequence = Options->GetMediaOption(ImgMedia::FillGapsInSequenceOption, true);
		LayerName = Options->GetMediaOption(ImgMedia::LayerNameOption, FString());

		TSharedPtr<IMediaOptions::FDataContainer, ESPMode::ThreadSafe> DefaultValue;
		TSharedPtr<IMediaOptions::FDataContainer, ESPMode::ThreadSafe> DataContainer =
			Options->GetMediaOption(ImgMedia::SourceColorSettingsOption, DefaultValue);
		if (DataContainer.IsValid())
		{
			SourceColorSettings = StaticCastSharedPtr<FNativeMediaSourceColorSettings, IMediaOptions::FDataContainer, ESPMode::ThreadSafe>(DataContainer);
		}
	}

	// initialize image loader on a separate thread
	FImgMediaLoaderSmartCacheSettings SmartCacheSettings(bIsSmartCacheEnabled, (float)SmartCacheTimeToLookAhead);
	Loader = MakeShared<FImgMediaLoader, ESPMode::ThreadSafe>(Scheduler.ToSharedRef(),
		GlobalCache.ToSharedRef(), SourceColorSettings, bFillGapsInSequence, LayerName, SmartCacheSettings,
		&EventSink, PlayerGuid);
	Scheduler->RegisterLoader(Loader.ToSharedRef());

	RebuildTileVisibilityProviders();

	// Subscribe to tracker change events so consumers that register after Open() (e.g. asset-
	// editor preview widgets opening over an active player) still get their providers picked up.
	AddMediaTextureTrackerEventHandlers();

	const FString SequencePath = UrlPath;

	Async(EAsyncExecution::ThreadPool, [FrameRateOverride, LoaderPtr = TWeakPtr<FImgMediaLoader, ESPMode::ThreadSafe>(Loader), Proxy, SequencePath, Loop = ShouldLoop, bIsSequencerEnvironment]()
	{
		TSharedPtr<FImgMediaLoader, ESPMode::ThreadSafe> PinnedLoader = LoaderPtr.Pin();

		if (PinnedLoader.IsValid())
		{
			FString ProxyPath = FPaths::Combine(SequencePath, Proxy);

			if (!FPaths::DirectoryExists(ProxyPath))
			{
				ProxyPath = SequencePath; // fall back to root folder
			}

			PinnedLoader->Initialize(ProxyPath, FrameRateOverride, Loop, bIsSequencerEnvironment);
		}
	}
#if WITH_EDITOR
	, [LoaderPtr = TWeakPtr<FImgMediaLoader, ESPMode::ThreadSafe>(Loader)]()
	{
		// Weak-capture only the Loader so this can't dangle FImgMediaPlayer* if the player is destroyed during loader init.
		TSharedPtr<FImgMediaLoader, ESPMode::ThreadSafe> PinnedLoader = LoaderPtr.Pin();
		if (PinnedLoader.IsValid() && PinnedLoader->IsInitialized() && PinnedLoader->GetNumImages() > 0)
		{
			RecordImgMediaSourceOpenedEvent(*PinnedLoader);
		}
	}
#endif
	);

	return true;
}

bool FImgMediaPlayer::Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& /*Archive*/, const FString& /*OriginalUrl*/, const IMediaOptions* /*Options*/)
{
	return false; // not supported
}

void FImgMediaPlayer::TickInput(FTimespan DeltaTime, FTimespan /*Timecode*/)
{
	SCOPE_CYCLE_COUNTER(STAT_ImgMedia_PlayerTickInput);

	if (!Loader.IsValid() || (CurrentState == EMediaState::Error))
	{
		return;
	}

	// Sync the loader's paused-refresh flag from the player state every tick. Anything that
	// isn't actively Playing - Stopped (e.g. "play on open" disabled), Paused, Preparing -
	// counts as paused for refresh purposes: the play head doesn't advance, so the displayed
	// frame's tiles should be allowed to refresh when visibility inputs change.
	Loader->SetIsPaused(CurrentState != EMediaState::Playing);

	Loader->TickFrame();

	// finalize loader initialization
	if ((CurrentState == EMediaState::Preparing) && Loader->IsInitialized())
	{
		if (Loader->GetSequenceDim().GetMin() == 0)
		{
			CurrentState = EMediaState::Error;

			EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);
		}
		else
		{
			CurrentDuration = Loader->GetSequenceDuration();
			CurrentState = EMediaState::Stopped;

			EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);
			EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpened);
		}
	}

	// Tick the scheduler an extra time in addition to its hookup as media clock sink, so we also get it moving forward during blocked playback
	Scheduler->TickInput(FTimespan::Zero(), FTimespan::MinValue());
}

bool FImgMediaPlayer::FlushOnSeekStarted() const
{
	return true;
}

bool FImgMediaPlayer::FlushOnSeekCompleted() const
{
	return false;
}

//-----------------------------------------------------------------------------
/**
	Get special feature flags states
*/
bool FImgMediaPlayer::GetPlayerFeatureFlag(EFeatureFlag flag) const
{
	switch (flag)
	{
	case EFeatureFlag::UsePlaybackTimingV2:
	case EFeatureFlag::PlayerUsesInternalFlushOnSeek:
		return true;
	default:
		break;
	}

	return IMediaPlayer::GetPlayerFeatureFlag(flag);
}

#if WITH_EDITOR
void FImgMediaPlayer::RecordSourceOpenedEvent() const
{
	if (Loader.IsValid())
	{
		RecordImgMediaSourceOpenedEvent(*Loader);
	}
}
#endif

/* FImgMediaPlayer implementation
 *****************************************************************************/

bool FImgMediaPlayer::IsInitialized() const
{
	return
		(CurrentState != EMediaState::Closed) &&
		(CurrentState != EMediaState::Error) &&
		(CurrentState != EMediaState::Preparing);
}


/* IMediaCache interface
 *****************************************************************************/

bool FImgMediaPlayer::QueryCacheState(EMediaCacheState State, TRangeSet<FTimespan>& OutTimeRanges) const
{
	if (!Loader.IsValid())
	{
		return false;
	}

	if (State == EMediaCacheState::Loading)
	{
		Loader->GetBusyTimeRanges(OutTimeRanges);
	}
	else if (State == EMediaCacheState::Loaded)
	{
		Loader->GetCompletedTimeRanges(OutTimeRanges);
	}
	else if (State == EMediaCacheState::Pending)
	{
		Loader->GetPendingTimeRanges(OutTimeRanges);
	}
	else
	{
		return false;
	}

	return true;
}


/* IMediaControls interface
 *****************************************************************************/

bool FImgMediaPlayer::CanControl(EMediaControl Control) const
{
	if (Control == EMediaControl::BlockOnFetch)
	{
		return true;
	}

	if (!IsInitialized())
	{
		return false;
	}

	if (Control == EMediaControl::Pause)
	{
		return (CurrentState == EMediaState::Playing);
	}

	if (Control == EMediaControl::Resume)
	{
		return (CurrentState != EMediaState::Playing);
	}

	if ((Control == EMediaControl::Scrub) || (Control == EMediaControl::Seek))
	{
		return true;
	}

	return false;
}


FTimespan FImgMediaPlayer::GetDuration() const
{
	return CurrentDuration;
}


float FImgMediaPlayer::GetRate() const
{
	return CurrentRate;
}


EMediaState FImgMediaPlayer::GetState() const
{
	return CurrentState;
}


EMediaStatus FImgMediaPlayer::GetStatus() const
{
	return EMediaStatus::None;
}


TRangeSet<float> FImgMediaPlayer::GetSupportedRates(EMediaRateThinning /*Thinning*/) const
{
	TRangeSet<float> Result;

	if (IsInitialized())
	{
		Result.Add(TRange<float>::Inclusive(-100000.0f, 100000.0f));
	}

	return Result;
}


FTimespan FImgMediaPlayer::GetTime() const
{
	return CurrentTime;
}


bool FImgMediaPlayer::IsLooping() const
{
	return ShouldLoop;
}


bool FImgMediaPlayer::Seek(const FTimespan& Time, const FMediaSeekParams& InAdditionalParams)
{
	// validate seek
	if (!IsInitialized())
	{
		UE_LOGF(LogImgMedia, Warning, "Cannot seek while player is not ready");
		return false;
	}

	if ((Time < FTimespan::Zero()) || (Time >= CurrentDuration))
	{
		UE_LOGF(LogImgMedia, Warning, "Invalid seek time %ls (media duration is %ls)", *Time.ToString(), *CurrentDuration.ToString());
		return false;
	}

	// scrub to desired time if needed
	if (CurrentState == EMediaState::Stopped)
	{
		CurrentState = EMediaState::Paused;
	}

	CurrentTime = Time;
	CurrentSeekIndex = InAdditionalParams.NewSequenceIndex.Get(0);
	if (Loader.IsValid())
	{
		Loader->Seek(FMediaTimeStamp(CurrentTime, CurrentSeekIndex, 0), LastNonZeroRate < 0.0f);
	}

	if (CurrentState == EMediaState::Paused)
	{
		Loader->RequestFrame(CurrentTime, CurrentRate, ShouldLoop);
	}

	LastFetchTime = FTimespan::MinValue();

	EventSink.ReceiveMediaEvent(EMediaEvent::SeekCompleted);

	return true;
}


bool FImgMediaPlayer::SetLooping(bool Looping)
{
	ShouldLoop = Looping;

	return true;
}


bool FImgMediaPlayer::SetRate(float Rate)
{
	if (!IsInitialized())
	{
		UE_LOGF(LogImgMedia, Warning, "Cannot set play rate while player is not ready");
		return false;
	}

	if (Rate == CurrentRate)
	{
		return true; // rate already set
	}

	if (CurrentDuration == FTimespan::Zero())
	{
		return false; // nothing to play
	}

	// handle restarting
	if ((CurrentRate == 0.0f) && (Rate != 0.0f))
	{
		if (CurrentState == EMediaState::Stopped)
		{
			PlaybackRestarted = true;
		}

		CurrentRate = Rate;
		LastNonZeroRate = Rate;
		CurrentState = EMediaState::Playing;

		EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackResumed);

		return true;
	}

	// handle pausing
	if ((CurrentRate != 0.0f) && (Rate == 0.0f))
	{
		LastNonZeroRate = CurrentRate;
		CurrentRate = Rate;
		CurrentState = EMediaState::Paused;

		EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackSuspended);
		return true;
	}

	if (Rate != 0.0f)
	{
		LastNonZeroRate = Rate;
	}
	CurrentRate = Rate;

	return true;
}

void FImgMediaPlayer::SetBlockingPlaybackHint(bool FacadeWillUseBlockingPlayback)
{
	if (PlaybackIsBlocking != FacadeWillUseBlockingPlayback)
	{
		PlaybackIsBlocking = FacadeWillUseBlockingPlayback;
		if (Loader.IsValid() && IsInitialized())
		{
			Loader->SetIsPlaybackBlocking(PlaybackIsBlocking);
		}
	}
}

/* IMediaSamples interface
 *****************************************************************************/

bool FImgMediaPlayer::FetchVideo(TRange<FTimespan> TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample)
{
	return false;
}


void FImgMediaPlayer::FlushSamples()
{
	if (IsInitialized())
	{
		CurrentSeekIndex = 0;
		LastFetchTime = FTimespan::MinValue();
		Loader->Flush();
	}
}


bool FImgMediaPlayer::DiscardVideoSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse)
{
	if (Loader.IsValid() && IsInitialized())
	{
		return Loader->DiscardVideoSamples(TimeRange, ShouldLoop, CurrentRate);
	}
	return false;
}

void FImgMediaPlayer::SetMinExpectedNextSequenceIndex(TOptional<int32> InNextSequenceIndex)
{
}


IMediaSamples::EFetchBestSampleResult FImgMediaPlayer::FetchBestVideoSampleForTimeRange(const TRange<FMediaTimeStamp>& TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample, bool bReverse, bool bConsistentResult)
{
	// note: the results produced by this player are "consistent" in respect to which frame is returned for a specific range, so we just disregard the bConsistentResult flag

	IMediaSamples::EFetchBestSampleResult SampleResult = EFetchBestSampleResult::NoSample;

	if (Loader.IsValid() && IsInitialized())
	{
		// See if we have any samples in the specified time range.
		SampleResult = Loader->FetchBestVideoSampleForTimeRange(TimeRange, OutSample, ShouldLoop, LastNonZeroRate);
		Scheduler->TickInput(FTimespan::Zero(), FTimespan::MinValue());
		if (SampleResult == IMediaSamples::EFetchBestSampleResult::Ok)
		{
			CurrentTime = OutSample->GetTime().Time;

			// Are we not looping?
			if (!ShouldLoop)
			{
				// Is this the last frame?
				if ((CurrentRate < 0.0f) ? Loader->IsFrameFirst(CurrentTime) : Loader->IsFrameLast(CurrentTime))
				{
					// Yes. Stop the player...
					EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackEndReached);
					CurrentState = EMediaState::Stopped;
					CurrentRate = 0.0f;
					LastNonZeroRate = 0.0f;
					EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackSuspended);
				}
			}
		}
	}
	return SampleResult;
}


bool FImgMediaPlayer::PeekVideoSampleTime(FMediaTimeStamp& TimeStamp)
{
	if (Loader.IsValid() && IsInitialized())
	{
		// Do we have the current frame?
		return Loader->PeekVideoSampleTime(TimeStamp, ShouldLoop, LastNonZeroRate, GetTime());
	}
	return false;
}

/* IMediaTracks interface
 *****************************************************************************/

bool FImgMediaPlayer::GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const
{
	return false; // not supported
}


int32 FImgMediaPlayer::GetNumTracks(EMediaTrackType TrackType) const
{
	return (Loader.IsValid() && (TrackType == EMediaTrackType::Video)) ? 1 : 0;
}


int32 FImgMediaPlayer::GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const
{
	return ((TrackIndex == 0) && (GetNumTracks(TrackType) > 0)) ? 1 : 0;
}


int32 FImgMediaPlayer::GetSelectedTrack(EMediaTrackType TrackType) const
{
	if (!IsInitialized() || (TrackType != EMediaTrackType::Video))
	{
		return INDEX_NONE;
	}

	return SelectedVideoTrack;
}


FText FImgMediaPlayer::GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	if (!IsInitialized() || (TrackType != EMediaTrackType::Video) || (TrackIndex != 0))
	{
		return FText::GetEmpty();
	}

	return LOCTEXT("DefaultVideoTrackName", "Video Track");
}


int32 FImgMediaPlayer::GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const
{
	return (GetSelectedTrack(TrackType) != INDEX_NONE) ? 0 : INDEX_NONE;
}


FString FImgMediaPlayer::GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const
{
	if (!IsInitialized() || (TrackType != EMediaTrackType::Video) || (TrackIndex != 0))
	{
		return FString();
	}

	return TEXT("und");
}


FString FImgMediaPlayer::GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	if (!IsInitialized() || (TrackType != EMediaTrackType::Video) || (TrackIndex != 0))
	{
		return FString();
	}

	return TEXT("VideoTrack");
}


bool FImgMediaPlayer::GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const
{
	if (!IsInitialized() || (TrackIndex != 0) || (FormatIndex != 0))
	{
		return false;
	}

	OutFormat.Dim = Loader->GetSequenceDim();
	OutFormat.FrameRate = Loader->GetSequenceFrameRate().AsDecimal();
	OutFormat.FrameRates = TRange<float>(OutFormat.FrameRate);
	OutFormat.TypeName = TEXT("Image"); // @todo gmp: fix me (should be image type)

	return true;
}


bool FImgMediaPlayer::SelectTrack(EMediaTrackType TrackType, int32 TrackIndex)
{
	if (!IsInitialized() || (TrackType != EMediaTrackType::Video))
	{
		return false;
	}

	if ((TrackIndex != 0) && (TrackIndex != INDEX_NONE))
	{
		return false;
	}

	SelectedVideoTrack = TrackIndex;

	return true;
}


bool FImgMediaPlayer::SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex)
{
	return (IsInitialized() && (TrackIndex == 0) && (FormatIndex == 0));
}

/* FImgMediaPlayer implementation
 *****************************************************************************/

void FImgMediaPlayer::AddMediaTextureTrackerEventHandlers()
{
	if (!TrackerObjectRegisteredHandle.IsValid())
	{
		TrackerObjectRegisteredHandle = FMediaTextureTracker::Get().OnObjectRegistered().AddLambda(
			[this](TObjectPtr<UMediaTexture> ChangedTexture, const FMediaTextureTrackerObjectPtr& /*ChangedInfo*/)
			{
				if (ChangedTexture == MediaTextureWeakPtr.Get())
				{
					RebuildTileVisibilityProviders();
				}
			});
	}

	if (!TrackerObjectUnregisteredHandle.IsValid())
	{
		TrackerObjectUnregisteredHandle = FMediaTextureTracker::Get().OnObjectUnregistered().AddLambda(
			[this](TObjectPtr<UMediaTexture> ChangedTexture, const FMediaTextureTrackerObjectPtr& /*ChangedInfo*/)
			{
				if (ChangedTexture == MediaTextureWeakPtr.Get())
				{
					RebuildTileVisibilityProviders();
				}
			});
	}
}

void FImgMediaPlayer::RemoveMediaTextureTrackerEventHandlers()
{
	if (TrackerObjectRegisteredHandle.IsValid())
	{
		FMediaTextureTracker::Get().OnObjectRegistered().Remove(TrackerObjectRegisteredHandle);
		TrackerObjectRegisteredHandle.Reset();
	}
	if (TrackerObjectUnregisteredHandle.IsValid())
	{
		FMediaTextureTracker::Get().OnObjectUnregistered().Remove(TrackerObjectUnregisteredHandle);
		TrackerObjectUnregisteredHandle.Reset();
	}
}

void FImgMediaPlayer::RebuildTileVisibilityProviders()
{
	// Two paths per tracker object:
	//   1. Consumer supplied an explicit TileVisibilityProvider - register it directly.
	//      This is how non-mesh consumers (e.g. preview viewports) plug in - they construct
	//      their own provider implementation and own its lifetime.
	//   2. Otherwise, dispatch on VisibleMipsTilesCalculations to construct a plane- or
	//      sphere-provider for legacy mesh consumers. The player owns those providers via
	//      OwnedTileVisibilityProviders.
	if (!Loader.IsValid())
	{
		return;
	}

	const TSharedPtr<FImgMediaTileVisibilityResolver, ESPMode::ThreadSafe>& Resolver =
		Loader->GetTileVisibilityResolver();
	Resolver->ClearProviders();
	OwnedTileVisibilityProviders.Reset();

	UMediaTexture* Texture = MediaTextureWeakPtr.Get();
	if (Texture == nullptr)
	{
		return;
	}

	const TArray<TWeakPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe>>* BoundObjects =
		FMediaTextureTracker::Get().GetObjects(Texture);
	if (BoundObjects == nullptr)
	{
		return;
	}

	TSharedPtr<FImgMediaSceneViewExtension, ESPMode::ThreadSafe> SceneViewExtension =
		IImgMediaModule::Get().GetSceneViewExtension();

	for (const TWeakPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe>& WeakTrackerObj : *BoundObjects)
	{
		TSharedPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe> TrackerObj = WeakTrackerObj.Pin();
		if (!TrackerObj.IsValid())
		{
			continue;
		}

		// Path 1: consumer supplied its own provider.
		if (TrackerObj->TileVisibilityProvider.IsValid())
		{
			Resolver->RegisterProvider(TrackerObj->TileVisibilityProvider.ToSharedRef());
			continue;
		}

		// Path 2: legacy enum-based mesh dispatch. Requires an actor with a mesh component.
		// todo: follow up task: extract those visibility providers and move them to MediaPlate module to make them available even when ImgMedia module is not present.
		AActor* OwnerActor = TrackerObj->Object.Get();
		if (OwnerActor == nullptr)
		{
			continue;
		}
		UMeshComponent* MeshComponent = Cast<UMeshComponent>(
			OwnerActor->FindComponentByClass(UMeshComponent::StaticClass()));
		if (MeshComponent == nullptr)
		{
			continue;
		}

		TSharedPtr<IMediaTileVisibilityProvider, ESPMode::ThreadSafe> Provider;
		switch (TrackerObj->VisibleMipsTilesCalculations)
		{
		case EMediaTextureVisibleMipsTiles::Plane:
		{
			FImgMediaPlaneVisibilityProviderParams ProviderParams;
			ProviderParams.MeshComponent = MeshComponent;
			ProviderParams.SceneViewExtension = SceneViewExtension;
			ProviderParams.MipMapLODBias = TrackerObj->MipMapLODBias;
			ProviderParams.MipLevelToUpscale = TrackerObj->MipLevelToUpscale;
			ProviderParams.TargetViewResolutionMask = TrackerObj->TargetViewResolutionMask;
			Provider = MakeShared<FImgMediaPlaneVisibilityProvider, ESPMode::ThreadSafe>(ProviderParams);
			break;
		}
		case EMediaTextureVisibleMipsTiles::Sphere:
		{
			FImgMediaSphereVisibilityProviderParams ProviderParams;
			ProviderParams.MeshComponent = MeshComponent;
			ProviderParams.SceneViewExtension = SceneViewExtension;
			ProviderParams.MipMapLODBias = TrackerObj->MipMapLODBias;
			ProviderParams.MipLevelToUpscale = TrackerObj->MipLevelToUpscale;
			ProviderParams.bAdaptivePoleMipUpscaling = TrackerObj->bAdaptivePoleMipUpscaling;
			ProviderParams.MeshRange = TrackerObj->MeshRange;
			ProviderParams.TargetViewResolutionMask = TrackerObj->TargetViewResolutionMask;
			Provider = MakeShared<FImgMediaSphereVisibilityProvider, ESPMode::ThreadSafe>(ProviderParams);
			break;
		}
		case EMediaTextureVisibleMipsTiles::None:
		default:
			break;
		}

		if (Provider.IsValid())
		{
			OwnedTileVisibilityProviders.Add(Provider);
			Resolver->RegisterProvider(Provider.ToSharedRef());
		}
	}
}

#undef LOCTEXT_NAMESPACE
