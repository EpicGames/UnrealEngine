// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IMediaTextureSample.h"
#include "Math/IntPoint.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"

#include "BaseMediaSourceColorSettings.h"
#include "Readers/IImgMediaReader.h"


/**
 * Texture sample generated by image sequence players.
 */
class FImgMediaTextureSample
	: public IMediaTextureSample
{
public:

	/** Default constructor. */
	FImgMediaTextureSample()
		: Duration(FTimespan::Zero())
		, OutputDim(FIntPoint::ZeroValue)
		, Time(FTimespan::Zero())
		, NumMipMaps(1)
		, TilingDesc()
		, SourceColorSettings()
	{ }

public:

	/**
	 * Initialize the sample.
	 *
	 * @param InFrame The image frame to create the sample for.
	 * @param InOutputDim The sample's output width and height (in pixels).
	 * @param InTime The sample time (in the player's local clock).
	 * @param InDuration The duration for which the sample is valid.
	 * @param InNumMipMaps The sample's number of mip levels.
	 * @param InTilingDesc The sample's tiling description.
	 * @param InSourceColorSettings The sample's source color space & encoding overrides.
	 */
	void Initialize(
		FImgMediaFrame& InFrame,
		const FIntPoint& InOutputDim,
		FMediaTimeStamp InTime,
		FTimespan InDuration,
		uint8 InNumMipMaps,
		FMediaTextureTilingDescription InTilingDesc,
		TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> InSourceColorSettings)
	{
		Duration = InDuration;
		Frame = InFrame;
		OutputDim = InOutputDim;
		Time = InTime;
		NumMipMaps = InNumMipMaps;
		TilingDesc = InTilingDesc;
		SourceColorSettings = InSourceColorSettings;

		// If we have no data then make sure the number of mipmaps is 1.
		// otherwise FMediaTextureResource won't like it.
		if ((Frame.Data.IsValid() == false) && (Frame.GetSampleConverter() == nullptr))
		{
			NumMipMaps = 1;
		}
	}

public:

	//~ IMediaTextureSample interface

	virtual const void* GetBuffer() override
	{
		return Frame.Data.Get();
	}

	virtual FIntPoint GetDim() const override
	{
		return Frame.GetDim();
	}

	virtual uint8 GetNumMips() const override
	{
		return NumMipMaps;
	}

	virtual FMediaTextureTilingDescription GetTilingDescription() const override
	{
		return TilingDesc;
	}

	virtual FTimespan GetDuration() const override
	{
		return Duration;
	}

	virtual EMediaTextureSampleFormat GetFormat() const override
	{
		return Frame.Format;
	}

	virtual FIntPoint GetOutputDim() const override
	{
		return OutputDim;
	}

	virtual uint32 GetStride() const override
	{
		return Frame.Stride;
	}

#if WITH_ENGINE
	virtual FRHITexture* GetTexture() const override
	{
		return nullptr;
	}
#endif //WITH_ENGINE

	virtual IMediaTextureSampleConverter* GetMediaTextureSampleConverter() override
	{
		return Frame.GetSampleConverter();
	}

	virtual FMediaTimeStamp GetTime() const override
	{
		return Time;
	}

	virtual bool IsCacheable() const override
	{
		return true;
	}

	virtual bool IsOutputSrgb() const override
	{
		if (SourceColorSettings.IsValid())
		{
			const UE::Color::EEncoding EncodingOverride = SourceColorSettings->GetEncodingOverride();

			if (EncodingOverride != UE::Color::EEncoding::None)
			{
				return EncodingOverride == UE::Color::EEncoding::sRGB;
			}
		}

		return Frame.IsOutputSrgb();
	}

	virtual const UE::Color::FColorSpace& GetSourceColorSpace() const override
	{
		// Like texture assets, we assume images are already in working color space.
		const UE::Color::FColorSpace& DefaultColorSpace = UE::Color::FColorSpace::GetWorking();

		if (SourceColorSettings.IsValid())
		{
			return SourceColorSettings->GetColorSpaceOverride(DefaultColorSpace);
		}

		return DefaultColorSpace;
	}

	virtual UE::Color::EEncoding GetEncodingType() const override
	{
		if (SourceColorSettings.IsValid())
		{
			const UE::Color::EEncoding EncodingOverride = SourceColorSettings->GetEncodingOverride();

			if (EncodingOverride != UE::Color::EEncoding::None)
			{
				return EncodingOverride;
			}
		}

		return IMediaTextureSample::GetEncodingType();
	}

	virtual UE::Color::EChromaticAdaptationMethod GetChromaticAdapationMethod() const override
	{
		if (SourceColorSettings.IsValid())
		{
			return SourceColorSettings->GetChromaticAdaptationMethod();
		}

		return IMediaTextureSample::GetChromaticAdapationMethod();
	}

private:

	/** Duration for which the sample is valid. */
	FTimespan Duration;

	/** The image frame that this sample represents. */
	FImgMediaFrame Frame;

	/** Width and height of the output. */
	FIntPoint OutputDim;

	/** Play time for which the sample was generated. */
	FMediaTimeStamp Time;

	/** Number of mip levels in this sample. */
	uint8 NumMipMaps;

	/** Description of the number and size of tiles in this sample. */
	FMediaTextureTilingDescription TilingDesc;

	/** Manual source color space & encoding overrides. */
	TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> SourceColorSettings;
};
