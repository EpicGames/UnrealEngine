// Copyright Epic Games, Inc. All Rights Reserved.

#include "DdsImgMediaReader.h"
#include "ImgMediaPrivate.h"

#include "ImageCoreUtils.h"
#include "Loader/ImgMediaLoader.h"
#include "Logging/StructuredLog.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

#include "DDSFile.h"

namespace ImgMedia
{
	namespace DDS
	{
		/** Descriptor for a DDS pixel format. */
		struct FDDSFormatInfo
		{
			UE::DDS::EDXGIFormat Format;
			uint32 Id;
			uint8 BytesPerUnit;
			bool bIsBlockCompressed;
		};

		static const FDDSFormatInfo FormatInfoList[] = 
		{
			#define RGBFMT(Name, Id, BytesPerUnit) { UE::DDS::EDXGIFormat::Name, Id, BytesPerUnit, false },
			#define BCNFMT(Name, Id, BytesPerUnit) { UE::DDS::EDXGIFormat::Name, Id, BytesPerUnit, true },
			#define ODDFMT(Name, Id) { UE::DDS::EDXGIFormat::Name, Id, 0, false },
			UE_DXGI_FORMAT_LIST
			#undef RGBFMT
			#undef BCNFMT
			#undef ODDFMT
		};

		static const FDDSFormatInfo* GetDDSFormatInfo(UE::DDS::EDXGIFormat InFormat)
		{
			// need to handle this special because UNKNOWN _does_ appear in the FormatInfoList,
			// but we don't want to treat it as legal
			if (InFormat == UE::DDS::EDXGIFormat::UNKNOWN)
			{
				return nullptr;
			}

			for (size_t Index = 0; Index < sizeof(FormatInfoList)/sizeof(*FormatInfoList); ++Index)
			{
				if (InFormat == FormatInfoList[Index].Format) 
				{
					return &FormatInfoList[Index];
				}
			}

			return nullptr;
		}

		bool IsFormatBlockCompressed(UE::DDS::EDXGIFormat InDXGIFormat)
		{
			const FDDSFormatInfo* FormatInfo = GetDDSFormatInfo(InDXGIFormat);
			return FormatInfo ? FormatInfo->bIsBlockCompressed : false;
		}
		
		/* Returns the media texture sample format from a DXGI format type. */
		bool GetMediaTextureSampleFormat(const UE::DDS::EDXGIFormat& InDXGIFormat, EMediaTextureSampleFormat& OutFormat, bool& bOutIsBlockCompressed)
		{
			switch (InDXGIFormat)
			{
			case UE::DDS::EDXGIFormat::BC1_TYPELESS:
			case UE::DDS::EDXGIFormat::BC1_UNORM:
			case UE::DDS::EDXGIFormat::BC1_UNORM_SRGB:
				OutFormat = EMediaTextureSampleFormat::DXT1;
				bOutIsBlockCompressed = true;
				break;
			case UE::DDS::EDXGIFormat::BC3_TYPELESS:
			case UE::DDS::EDXGIFormat::BC3_UNORM:
			case UE::DDS::EDXGIFormat::BC3_UNORM_SRGB:
				OutFormat = EMediaTextureSampleFormat::DXT5;
				bOutIsBlockCompressed = true;
				break;
			case UE::DDS::EDXGIFormat::BC4_TYPELESS:
			case UE::DDS::EDXGIFormat::BC4_UNORM:
			case UE::DDS::EDXGIFormat::BC4_SNORM:
				OutFormat = EMediaTextureSampleFormat::BC4;
				bOutIsBlockCompressed = true;
				break;
			case UE::DDS::EDXGIFormat::B8G8R8A8_TYPELESS:
			case UE::DDS::EDXGIFormat::B8G8R8A8_UNORM:
			case UE::DDS::EDXGIFormat::R8G8B8A8_SNORM:
			case UE::DDS::EDXGIFormat::B8G8R8A8_UNORM_SRGB:
				OutFormat = EMediaTextureSampleFormat::CharBGRA;
				bOutIsBlockCompressed = false;
				break;
			case UE::DDS::EDXGIFormat::R8G8B8A8_TYPELESS:
			case UE::DDS::EDXGIFormat::R8G8B8A8_UNORM:
			case UE::DDS::EDXGIFormat::R8G8B8A8_UNORM_SRGB:
				OutFormat = EMediaTextureSampleFormat::CharRGBA;
				bOutIsBlockCompressed = false;
				break;
			case UE::DDS::EDXGIFormat::R16G16B16A16_FLOAT:
				OutFormat = EMediaTextureSampleFormat::FloatRGBA;
				bOutIsBlockCompressed = false;
				break;

			default:
				UE_LOGF(LogImgMedia, Error, "Unsupported compression format, only BC1/DXT1, BC3/DXT5, BC4 & FloatRGBA are currently supported.");
				OutFormat = EMediaTextureSampleFormat::Undefined;
				bOutIsBlockCompressed = false;
				return false;
			}

			return true;
		}

		/*
		 * For block compressed formats, only consider mips larger than the block size as valid. 
		 */
		inline bool IsMipValid(const UE::DDS::FDDSMip& InMip, bool bIsBlockCompressed)
		{
			if (bIsBlockCompressed)
			{
				// See FD3D12Texture::UpdateTexture2D block size checks, and also
				// TODO: D3D12_FEATURE_DATA_D3D12_OPTIONS8.UnalignedBlockTexturesSupported
				return FMath::Min(InMip.Width, InMip.Height) >= 4 /* CompressionBlockSize */;
			}

			return true;
		}

		/* Returns the DDS total mip data size. */
		SIZE_T GetTotalMipDataSize(const TUniquePtr<UE::DDS::FDDSFile>& DDS)
		{
			const bool bIsBlockCompressed = IsFormatBlockCompressed(DDS->DXGIFormat);
			SIZE_T TotalMipDataSize = 0;
			for (const UE::DDS::FDDSMip& Mip : DDS->Mips)
			{
				if (!IsMipValid(Mip, bIsBlockCompressed))
				{
					break;
				}

				TotalMipDataSize += static_cast<SIZE_T>(Mip.DataSize);
			}

			return TotalMipDataSize;
		}

		/* Returns the DDS file mip count, accounting for engine maximum. */
		int32 GetMipCount(const TUniquePtr<UE::DDS::FDDSFile>& DDS)
		{
			const bool bIsBlockCompressed = IsFormatBlockCompressed(DDS->DXGIFormat);
			int32 MipCount = 0;

			// Mips are ordered starting from mip 0 (full-size texture) decreasing in size
			for (const UE::DDS::FDDSMip& Mip : DDS->Mips)
			{
				if (!IsMipValid(Mip, bIsBlockCompressed))
				{
					break;
				}

				++MipCount;
			}

			return FMath::Min(MipCount, static_cast<int32>(MAX_TEXTURE_MIP_COUNT));
		}

		/** Returns true if the given format is a floating point type. */
		bool IsFloatFormat(UE::DDS::EDXGIFormat InFormat)
		{
			switch (InFormat)
			{
			case UE::DDS::EDXGIFormat::R32G32B32A32_FLOAT:
			case UE::DDS::EDXGIFormat::R32G32B32_FLOAT:
			case UE::DDS::EDXGIFormat::R16G16B16A16_FLOAT:
			case UE::DDS::EDXGIFormat::R32G32_FLOAT:
			case UE::DDS::EDXGIFormat::R11G11B10_FLOAT:
			case UE::DDS::EDXGIFormat::R16G16_FLOAT:
			case UE::DDS::EDXGIFormat::D32_FLOAT:
			case UE::DDS::EDXGIFormat::R32_FLOAT:
			case UE::DDS::EDXGIFormat::R16_FLOAT:
				return true;
			default:
				return false;
			}
		}

		/* Returns whether the DDS file is sRGB-encoded. */
		bool DDSPayloadIsSRGB(const TUniquePtr<UE::DDS::FDDSFile>& DDS)
		{
			// Float formats are considered linear.
			if (IsFloatFormat(DDS->DXGIFormat))
			{
				return false;
			}
			
			if (DDS->CreateFlags & UE::DDS::FDDSFile::CREATE_FLAG_WAS_D3D9)
			{
				// no SRGB info in Dx9 format
				// assume SRGB yes
				return true;
			}
			else if (UE::DDS::DXGIFormatHasLinearAndSRGBForm(DDS->DXGIFormat))
			{
				// Dx10 file with format that has linear/srgb pair
				//	( _UNORM when _UNORM_SRGB)
				return UE::DDS::DXGIFormatIsSRGB(DDS->DXGIFormat);
			}
			else
			{
				// Dx10 format that doesn't have linear/srgb pairs

				// R8G8_UNORM and R8_UNORM have no _SRGB pair
				// so no way to clearly indicate SRGB or Linear for them
				// assume SRGB yes
				return true;
			}
		}

		/* Returns DDS file frame information for image sequence playback. */
		FImgMediaFrameInfo GetFrameInfo(const TUniquePtr<UE::DDS::FDDSFile>& DDS)
		{
			FImgMediaFrameInfo OutInfo;

			OutInfo.CompressionName = UE::DDS::DXGIFormatGetName(DDS->DXGIFormat);
			OutInfo.Dim.X = DDS->Width;
			OutInfo.Dim.Y = DDS->Height;
			OutInfo.UncompressedSize = GetTotalMipDataSize(DDS);
			OutInfo.NumMipLevels = GetMipCount(DDS);
			OutInfo.FormatName = TEXT("DDS");
			OutInfo.FrameRate = GetDefault<UImgMediaSettings>()->DefaultFrameRate;
			OutInfo.Srgb = DDSPayloadIsSRGB(DDS);
			OutInfo.NumChannels = 4;
			OutInfo.bHasTiles = false;
			OutInfo.TileDimensions = OutInfo.Dim;
			OutInfo.NumTiles = FIntPoint(1, 1);
			OutInfo.TileBorder = 0;

			return OutInfo;
		}
	}
}

/* Convenience class for reading header/mip information and raw data from DDS files. */
class FDdsReader
{
public:
	/* Constructor */
	FDdsReader(const FString& InFilename)
		: Filename(InFilename)
		, ArFileReader(IFileManager::Get().CreateFileReader(*InFilename))
		, StartSeekPos(0)
	{
	}

	/* Read file header and prepare for raw data reads. Must be called in advance of ReadRawMipData. */
	TUniquePtr<UE::DDS::FDDSFile> ReadHeaderAndPrepare()
	{
		TArray64<uint8> HeaderData;
		if (!ReadHeaderData(HeaderData))
		{
			return nullptr;
		}

		UE::DDS::EDDSError Error;
		TUniquePtr<UE::DDS::FDDSFile> DDS = TUniquePtr<UE::DDS::FDDSFile>(UE::DDS::FDDSFile::CreateFromDDSInMemory(HeaderData.GetData(), HeaderData.Num(), &Error, UE::DDS::EDDSReadMipMode::HeaderWithMipInfo));
		if (!DDS.IsValid())
		{
			check(Error != UE::DDS::EDDSError::OK);
			if (Error != UE::DDS::EDDSError::NotADds && Error != UE::DDS::EDDSError::IoError)
			{
				UE_LOGF(LogImgMedia, Warning, "Failed to load DDS (Error=%d) [%ls]", (int)Error, *Filename);
			}
			
			return nullptr;
		}

		const int64 SizeOfFile = ArFileReader->TotalSize();
		int64 TotalMipDataSize = 0;
		for (const UE::DDS::FDDSMip& Mip : DDS->Mips)
		{
			TotalMipDataSize += Mip.DataSize;
		}
		StartSeekPos = SizeOfFile - TotalMipDataSize;
		
		return MoveTemp(DDS);
	}

	/* Read a specified size of raw file data. */
	void ReadRawMipData(int64 MipDataOffset, void* MipData, int64 MipDataSize) const
	{
		if (ArFileReader.IsValid())
		{
			ArFileReader->Seek(StartSeekPos + MipDataOffset);
			ArFileReader->Serialize(MipData, MipDataSize);
		}
	}

	/* Close the file reader. */
	void Close()
	{
		if (ArFileReader.IsValid())
		{
			ArFileReader->Close();
		}
	}

private:

	/* Convenience function to read the header data. */
	bool ReadHeaderData(TArray64<uint8>& OutHeader) const
	{
		if (!ArFileReader.IsValid())
		{
			return false;
		}

		const int64 SizeOfFile = ArFileReader->TotalSize();
		const int64 MinimalHeaderSize = UE::DDS::GetDDSHeaderMinimalSize();

		// If the file is not bigger than the smallest header possible then clearly the file is not valid as a dds file.
		if (SizeOfFile > MinimalHeaderSize)
		{
			const int64 MaximalHeaderSize = UE::DDS::GetDDSHeaderMaximalSize();
			const int64 BytesToRead = FMath::Min(SizeOfFile, MaximalHeaderSize);

			OutHeader.Reset(BytesToRead);
			OutHeader.AddUninitialized(BytesToRead);
			ArFileReader->Serialize(OutHeader.GetData(), OutHeader.Num());

			return true;
		}

		return false;
	}

private:

	// Loaded file name
	FString Filename;
	
	// File reader
	TUniquePtr<FArchive> ArFileReader;

	// Start seek position for mip data
	int64 StartSeekPos;
};


/* FGenericImgMediaReader structors
 *****************************************************************************/

FDdsImgMediaReader::FDdsImgMediaReader(const TSharedRef<FImgMediaLoader, ESPMode::ThreadSafe>& InLoader)
	: LoaderPtr(InLoader)
{ }

FDdsImgMediaReader::~FDdsImgMediaReader() = default;

/* FDdsImgMediaReader interface
 *****************************************************************************/

bool FDdsImgMediaReader::GetFrameInfo(const FString& ImagePath, FImgMediaFrameInfo& OutInfo)
{
	FDdsReader Reader(ImagePath);
	TUniquePtr<UE::DDS::FDDSFile> DDS = Reader.ReadHeaderAndPrepare();
	if (!DDS.IsValid())
	{
		UE_LOGFMT(LogImgMedia, Warning, "FDdsImgMediaReader: Failed to load image {0}", ImagePath);
		return false;
	}

	EMediaTextureSampleFormat SampleFormat;
	bool bIsBlockCompressed;
	if (!ImgMedia::DDS::GetMediaTextureSampleFormat(DDS->DXGIFormat, SampleFormat, bIsBlockCompressed))
	{
		UE_LOGFMT(LogImgMedia, Error, "FDdsImgMediaReader: Texture format {0} is not currently supported.", UE::DDS::DXGIFormatGetName(DDS->DXGIFormat));
		return false;
	}

	const int32 MipCount = ImgMedia::DDS::GetMipCount(DDS);
	if (MipCount > 1)
	{
		if (bIsBlockCompressed && (!FMath::IsPowerOfTwo(DDS->Width) || !FMath::IsPowerOfTwo(DDS->Height)))
		{
			UE_LOGFMT(LogImgMedia, Error, "FDdsImgMediaReader: Compressed textures with mip maps currently need to have power-of-two dimensions.");
			return false;
		}
	}

	OutInfo = ImgMedia::DDS::GetFrameInfo(DDS);

	return true;
}


bool FDdsImgMediaReader::ReadFrame(int32 FrameId, const TMap<int32, FMediaTileSelection>& InMipTiles, TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> OutFrame)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("DdsImgMedia.ReadFrame %d"), FrameId));

	const TSharedPtr<FImgMediaLoader, ESPMode::ThreadSafe> Loader = LoaderPtr.Pin();
	if (!Loader.IsValid())
	{
		return false;
	}

	if (InMipTiles.IsEmpty())
	{
		return false;
	}

	const FString& ImagePath = Loader->GetImagePath(FrameId, 0);
	FDdsReader Reader(ImagePath);

	TUniquePtr<UE::DDS::FDDSFile> DDS = Reader.ReadHeaderAndPrepare();
	if (!DDS.IsValid())
	{
		UE_LOGF(LogImgMedia, Warning, "FDdsImgMediaReader: Failed to load image %ls", *ImagePath);
		return false;
	}

	int64 MipDataOffset = 0;
	const int32 MipCount = ImgMedia::DDS::GetMipCount(DDS);

	// Loop over all mips.
	for (int32 MipLevel = 0; MipLevel < MipCount; ++MipLevel)
	{
		const int64 MipDataSize = DDS->Mips[MipLevel].DataSize;
		const bool bMipLevelRequested = InMipTiles.Contains(MipLevel);
		const bool bMipLevelCached = OutFrame->MipTilesPresent.Contains(MipLevel);

		if (bMipLevelRequested && !bMipLevelCached)
		{
			const bool bBufferAllocated = OutFrame->Data.IsValid();

			if (!bBufferAllocated)
			{
				const SIZE_T TotalMipDataSize = ImgMedia::DDS::GetTotalMipDataSize(DDS);
				void* Buffer = FMemory::Malloc(TotalMipDataSize, PLATFORM_CACHE_LINE_SIZE);

				EMediaTextureSampleFormat SampleFormat;
				bool bIsBlockCompressed;
				ImgMedia::DDS::GetMediaTextureSampleFormat(DDS->DXGIFormat, SampleFormat, bIsBlockCompressed);

				OutFrame->Data = MakeShareable(Buffer, [](void* ObjectToDelete) { FMemory::Free(ObjectToDelete); });
				OutFrame->Stride = DDS->Mips[0].RowStride;
				OutFrame->SetInfo(ImgMedia::DDS::GetFrameInfo(DDS));
				OutFrame->Format = SampleFormat;

				OutFrame->MipTilesPresent.Reset();
			}

			void* MipData = static_cast<void*>((uint8*)OutFrame->Data.Get() + static_cast<SIZE_T>(MipDataOffset));
			Reader.ReadRawMipData(MipDataOffset, MipData, MipDataSize);

			OutFrame->MipTilesPresent.Emplace(MipLevel, InMipTiles[MipLevel]);
			OutFrame->NumTilesRead++;
		}

		MipDataOffset += MipDataSize;
	}

	/*
	* TODO: We currently rely on the media texture copy sample logic to update the
	* texture resource mips, which needlessly copies mips that have not been read.
	*
	* Using our own sample converter (like FExrMediaTextureSampleConverter would solve
	* this issue if only the media texture resource did not force conversion to float on
	* DXT formats. See GetConvertedPixelFormat & FMediaTextureResource::RequiresConversion.
	* 
	* Changing these functions without further refactoring to allow PF_DXT1 & PF_DXT5 would
	* currently break other players however.
	*/

	return true;
}
