// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "IImgMediaReader.h"

class FImgMediaLoader;

/**
 * Implements a reader for various image sequence formats.
 */
class FDdsImgMediaReader final
	: public IImgMediaReader
{
public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InImageWrapperModule The image wrapper module to use.
	 */
	FDdsImgMediaReader(const TSharedRef<FImgMediaLoader, ESPMode::ThreadSafe>& InLoader);

	~FDdsImgMediaReader();

public:

	//~ IImgMediaReader interface

	virtual bool GetFrameInfo(const FString& ImagePath, FImgMediaFrameInfo& OutInfo) override;
	virtual bool ReadFrame(int32 FrameId, const TMap<int32, FMediaTileSelection>& InMipTiles, TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> OutFrame) override;
	virtual void CancelFrame(int32 FrameNumber) override {};
	virtual void UncancelFrame(int32 FrameNumber) override {};

private:

	/** Our parent loader. */
	TWeakPtr<FImgMediaLoader, ESPMode::ThreadSafe> LoaderPtr;
};

