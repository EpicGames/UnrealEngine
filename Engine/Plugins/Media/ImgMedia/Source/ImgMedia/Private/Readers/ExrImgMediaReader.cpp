// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExrImgMediaReader.h"
#include "ExrImgMediaReaderGpu.h"

#if IMGMEDIA_EXR_SUPPORTED_PLATFORM

#include "Assets/MediaTileSelection.h"
#include "Async/Async.h"
#include "ExrReaderGpu.h"
#include "HardwareInfo.h"
#include "IImgMediaModule.h"
#include "Loader/ImgMediaLoader.h"
#include "Logging/StructuredLog.h"
#include "Misc/Paths.h"
#include "OpenExrWrapper.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"

DECLARE_MEMORY_STAT(TEXT("EXR Reader Pool Memory."), STAT_ExrMediaReaderPoolMem, STATGROUP_ImgMediaPlugin);

static TAutoConsoleVariable<bool> CVarEnableUncompressedExrGpuReader(
	TEXT("r.ExrReadAndProcessOnGPU"),
	true,
	TEXT("Allows reading of Large Uncompressed EXR files directly into Structured Buffer.\n")
	TEXT("and be processed on GPU\n"));


/* FExrImgMediaReader structors
 *****************************************************************************/

FExrImgMediaReader::FExrImgMediaReader(const TSharedRef<FImgMediaLoader, ESPMode::ThreadSafe>& InLoader, const FString& InLayerName)
	: LoaderPtr(InLoader)
	, LayerName(InLayerName)
{
	const UImgMediaSettings* Settings = GetDefault<UImgMediaSettings>();
	
	FOpenExr::SetGlobalThreadCount(Settings->ExrDecoderThreads == 0
		? FPlatformMisc::NumberOfCoresIncludingHyperthreads()
		: Settings->ExrDecoderThreads);
}

FExrImgMediaReader::~FExrImgMediaReader()
{
	FScopeLock RegionScopeLock(&CanceledFramesCriticalSection);
	CanceledFrames.Reset();
}

/* FExrImgMediaReader interface
 *****************************************************************************/

bool FExrImgMediaReader::GetFrameInfo(const FString& ImagePath, FImgMediaFrameInfo& OutInfo)
{
	if (!GetInfo(ImagePath, OutInfo))
	{
		return false;
	}

	// Only read mip 0; in-file multi-resolution levels not supported
	// todo: Implement Imf::TiledRgbaInputFile::readTile to relax this.
	if (OutInfo.NumMipLevels > 1)
	{
		OutInfo.NumMipLevels = 1;
		OutInfo.UncompressedSize = GetMipBufferTotalSize(OutInfo.Dim, /*bInHasMips*/ false);
	}

	return true;
}

FExrImgMediaReader::EReadResult FExrImgMediaReader::ReadTiles
	( uint16* Buffer
	, int64 BufferSize
	, const FString& ImagePath
	, const TArray<FIntRect>& TileRegions
	, FSampleConverterParameters& ConverterParams
	, const int32 CurrentMipLevel
	, TArray<UE::Math::TIntPoint<int64>>& OutBufferRegionsToCopy)
{
#if defined(PLATFORM_WINDOWS) && PLATFORM_WINDOWS
	EReadResult Result = Success;

	FExrReader ChunkReader;
	int MipLevelDiv = 1 << CurrentMipLevel;

	const FIntPoint MipResolution = (ConverterParams.FullResolution / MipLevelDiv).ComponentMax(FIntPoint(1,1));

	FIntPoint DimensionInTiles
		( FMath::CeilToInt(static_cast<float>(MipResolution.X) / ConverterParams.TileDimWithBorders.X)
		, FMath::CeilToInt(static_cast<float>(MipResolution.Y) / ConverterParams.TileDimWithBorders.Y));

	TArray<int32> NumTilesPerLevel;
	int32 NumMipLevels = ConverterParams.bMipsInSeparateFiles ? 1 : ConverterParams.NumMipLevels;

	FExrReader::CalculateTileOffsets(
		NumTilesPerLevel,
		ConverterParams.TileInfoPerMipLevel,
		ConverterParams.FullResolution,
		ConverterParams.TileDimWithBorders,
		NumMipLevels,
		ConverterParams.PixelSize);

	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("FExrImgMediaReader_ReadTiles_OpenFile");
		if (!ChunkReader.OpenExrAndPrepareForPixelReading(ImagePath, NumTilesPerLevel))
		{
			return Fail;
		}
	}
	{
		int64 CurrentBufferPos = 0;
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("FExrImgMediaReader_ReadTiles_ReadTiles");
		for (const FIntRect& RawTileRegion : TileRegions)
		{
			// This clamp is to make sure that tile region is not out of bounds in case the region wasn't calculated incorrectly for some reason.
			const FIntRect& TileRegion = FIntRect(
				FIntPoint(FMath::Max(RawTileRegion.Min.X, 0), FMath::Max(RawTileRegion.Min.Y, 0)),
				FIntPoint(FMath::Min(RawTileRegion.Max.X, DimensionInTiles.X), FMath::Min(RawTileRegion.Max.Y, DimensionInTiles.Y)));

			for (int32 TileRow = TileRegion.Min.Y; TileRow < TileRegion.Max.Y; TileRow++)
			{
				// Check to see if the frame was canceled.
				{
					FScopeLock RegionScopeLock(&CanceledFramesCriticalSection);
					if (CanceledFrames.Remove(ConverterParams.FrameId) > 0)
					{
						UE_LOGF(LogImgMedia, Verbose, "Reader %p: Canceling Frame %i At tile row # %i", this, ConverterParams.FrameId, TileRow);
						Result = Cancelled;
						break;
					}
				}

				const uint16 Padding = FExrReader::TILE_PADDING;
				const int StartTileIndex = TileRow * DimensionInTiles.X + TileRegion.Min.X;

				bool bLastTile = TileRow + 1 == DimensionInTiles.Y && TileRegion.Max.X == DimensionInTiles.X;
				const int EndTileIndex = TileRow * DimensionInTiles.X + TileRegion.Max.X;

				int MipLevel = ConverterParams.bMipsInSeparateFiles ? 0 : CurrentMipLevel;
				ChunkReader.SeekTileWithinFile(StartTileIndex, MipLevel, CurrentBufferPos);
				int64 ByteOffsetStart = 0;
				int64 ByteOffsetEnd = 0;

				if (!ChunkReader.GetByteOffsetForTile(StartTileIndex, MipLevel, ByteOffsetStart)
					|| !ChunkReader.GetByteOffsetForTile(EndTileIndex, MipLevel, ByteOffsetEnd))
				{
					Result = Fail;
					break;
				}

				// If this is the last tile, make sure chunk reader reads till the end of the buffer we write into.
				int64 ByteChunkToRead = bLastTile
					? FMath::Min(ByteOffsetEnd - ByteOffsetStart, BufferSize - CurrentBufferPos)
					: (ByteOffsetEnd - ByteOffsetStart);

				if (!ChunkReader.ReadExrImageChunk(reinterpret_cast<char*>(Buffer) + CurrentBufferPos, ByteChunkToRead))
				{
					Result = Fail;
					break;
				}
				OutBufferRegionsToCopy.Add({ CurrentBufferPos, ByteChunkToRead });
			}

		}
	}
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("FExrImgMediaReaderGpu_CloseFile %d"), CurrentMipLevel));

		if (!ChunkReader.CloseExrFile())
		{
			return Fail;
		}

		return Result;
	}
#else
	return Fail;
#endif
}

bool FExrImgMediaReader::ReadFrame(int32 FrameId, const TMap<int32, FMediaTileSelection>& InMipTiles, TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> OutFrame)
{
	TSharedPtr<FImgMediaLoader, ESPMode::ThreadSafe> Loader = LoaderPtr.Pin();
	if (Loader.IsValid() == false)
	{
		return false;
	}

	/**
	 * Resolve layer on first read (only when a layer name is configured).
	 * Uses the double-checked locking pattern: the acquire load provides a
	 * lock-free fast path for all frames after the first, while the inner
	 * relaxed load re-checks under the lock to avoid redundant resolution
	 * if two threads race on the very first frame.
	 */
	if (!LayerName.IsEmpty() && !bLayerResolved.load(std::memory_order_acquire))
	{
		FScopeLock RegionScopeLock(&LayerResolveCriticalSection);
		if (!bLayerResolved.load(std::memory_order_relaxed))
		{
			const FString& FirstImage = Loader->GetImagePath(FrameId, 0);
			if (!ResolveLayer(FirstImage))
			{
				UE_LOGFMT(LogImgMedia, Error, "Failed to resolve EXR layer '{0}'", LayerName);
				return false;
			}
		}
	}

	int32 BytesPerPixelPerChannel = sizeof(uint16);
	int32 NumChannels = 4;
	int32 BytesPerPixel = BytesPerPixelPerChannel * NumChannels;
	FImgMediaFrameInfo FrameInfo;

	// Do we already have our buffer?
	if (OutFrame->Data.IsValid() == false)
	{
		// Nope. Create it.
		const FString& LargestImage = Loader->GetImagePath(FrameId, 0);
		if (!GetInfo(LargestImage, FrameInfo, OutFrame))
		{
			return false;
		}

		const FIntPoint& Dim = FrameInfo.Dim;

		if (Dim.GetMin() <= 0)
		{
			return false;
		}

		/** Allocate the frame buffer. FExrInputFile always outputs interleaved RGBA 16-bit half-float,
		 *  so the buffer size is Dim.X * Dim.Y * 4 * sizeof(uint16) per mip level.
		 *  UncompressedSize must reflect the actual allocation size as it is used for cache accounting.
		 */
		FrameInfo.UncompressedSize = GetMipBufferTotalSize(Dim, Loader->GetNumMipLevels() > 1);
		SIZE_T BufferSize = FrameInfo.UncompressedSize;
		void* Buffer = FMemory::Malloc(BufferSize, PLATFORM_CACHE_LINE_SIZE);

		auto BufferDeleter = [BufferSize](void* ObjectToDelete) {
#if USE_IMGMEDIA_DEALLOC_POOL
			if (FQueuedThreadPool* ImgMediaThreadPoolSlow = GetImgMediaThreadPoolSlow())
			{
				// free buffers on the thread pool, because memory allocators may perform
				// expensive operations, such as filling the memory with debug values
				TFunction<void()> FreeBufferTask = [ObjectToDelete]()
				{
					FMemory::Free(ObjectToDelete);
				};
				AsyncPool(*ImgMediaThreadPoolSlow, FreeBufferTask);
			}
			else
			{
				FMemory::Free(ObjectToDelete);
			}
#else
			FMemory::Free(ObjectToDelete);
#endif
		};
		
		// The EXR RGBA interface only outputs RGBA data.
		OutFrame->Format = EMediaTextureSampleFormat::FloatRGBA;
		OutFrame->Data = MakeShareable(Buffer, MoveTemp(BufferDeleter));
		OutFrame->MipTilesPresent.Reset();
		OutFrame->Stride = FrameInfo.Dim.X * BytesPerPixel;
	}
	else
	{
		FrameInfo = OutFrame->GetInfo();
	}

	// Loop over all mips.
	uint8* MipDataPtr = (uint8*)(OutFrame->Data.Get());
	FIntPoint Dim = FrameInfo.Dim;
	
	int32 NumMipLevels = Loader->GetNumMipLevels();
	for (int32 CurrentMipLevel = 0; CurrentMipLevel < NumMipLevels; ++CurrentMipLevel)
	{
		if (InMipTiles.Contains(CurrentMipLevel))
		{
			const FMediaTileSelection& CurrentTileSelection = InMipTiles[CurrentMipLevel];

			const int MipLevelDiv = 1 << CurrentMipLevel;

			// Avoid reads if the cached frame already contains the current tiles for this mip level.
			const bool ReadThisMip = !OutFrame->MipTilesPresent.ContainsTiles(CurrentMipLevel, CurrentTileSelection);
			if (ReadThisMip)
			{
				FString Image = Loader->GetImagePath(FrameId, CurrentMipLevel);
				FString BaseImage;

				if (LayerName.IsEmpty())
				{
					FRgbaInputFile InputFile(Image, 2);
					if (InputFile.HasInputFile())
					{
						InputFile.SetFrameBuffer(MipDataPtr, Dim);
						InputFile.ReadPixels(0, Dim.Y - 1);
						OutFrame->MipTilesPresent.Emplace(CurrentMipLevel, CurrentTileSelection);
						OutFrame->NumTilesRead++;
					}
					else
					{
						UE_LOGFMT(LogImgMedia, Error, "Could not load {0}", Image);
					}
				}
				else
				{
					FExrInputFile InputFile(Image, 2);
					if (InputFile.HasInputFile())
					{
						InputFile.ReadPixels(
							MipDataPtr, Dim,
							ResolvedChannelR, ResolvedChannelG,
							ResolvedChannelB, ResolvedChannelA,
							0, Dim.Y - 1);
						OutFrame->MipTilesPresent.Emplace(CurrentMipLevel, CurrentTileSelection);
						OutFrame->NumTilesRead++;
					}
					else
					{
						UE_LOGFMT(LogImgMedia, Error, "Could not load {0}", Image);
					}
				}
			}
		}

		// Next level.
		MipDataPtr += Dim.X * Dim.Y * BytesPerPixel;
		Dim /= 2;
	}

	return true;
}

void FExrImgMediaReader::CancelFrame(int32 FrameNumber)
{
	UE_LOGF(LogImgMedia, Verbose, "Reader %p: Canceling Frame. %i", this, FrameNumber);
	FScopeLock RegionScopeLock(&CanceledFramesCriticalSection);
	CanceledFrames.Add(FrameNumber);
}

void FExrImgMediaReader::UncancelFrame(int32 FrameNumber)
{
	FScopeLock RegionScopeLock(&CanceledFramesCriticalSection);
	CanceledFrames.Remove(FrameNumber);
}

/** Gets reader type (GPU vs CPU) depending on size of EXR and its compression. */
TSharedPtr<IImgMediaReader, ESPMode::ThreadSafe> FExrImgMediaReader::GetReader(const TSharedRef <FImgMediaLoader, ESPMode::ThreadSafe>& InLoader, FString FirstImageInSequencePath, const FString& InLayerName)
{
#if defined(PLATFORM_WINDOWS) && PLATFORM_WINDOWS
	if (!FPaths::FileExists(FirstImageInSequencePath))
	{
		return nullptr;
	}
	
	FImgMediaFrameInfo Info;
	if (!GetInfo(FirstImageInSequencePath, Info))
	{
		return MakeShareable(new FExrImgMediaReader(InLoader, InLayerName));
	}

	const bool bIsDevelopmentCustomFormat = Info.FormatName.Equals(TEXT("EXR CUSTOM"));
	if (bIsDevelopmentCustomFormat)
	{
		UE_LOGF(LogImgMedia, Error, "Support for the (development-only) custom EXR format has been removed.");
		return nullptr;
	}

	const bool bIsOptimizedForGpu = Info.FormatName.Equals(TEXT("EXR GPU"));
	// Check GetCompressionName of OpenExrWrapper for other compression names.
	// todo: Add and test Vulkan support
	if (GDynamicRHI && GDynamicRHI->GetInterfaceType() == ERHIInterfaceType::D3D12
		&& Info.CompressionName == "Uncompressed"
		&& CVarEnableUncompressedExrGpuReader.GetValueOnAnyThread()
		&& bIsOptimizedForGpu
		)
	{
		TSharedRef<FExrImgMediaReaderGpu, ESPMode::ThreadSafe> GpuReader =
			MakeShared<FExrImgMediaReaderGpu, ESPMode::ThreadSafe>(InLoader, InLayerName);
		return GpuReader;
	}
#endif

	return MakeShareable(new FExrImgMediaReader(InLoader, InLayerName));
}

bool FExrImgMediaReader::ResolveLayer(const FString& ImagePath)
{
	if (bLayerResolved.load(std::memory_order_relaxed))
	{
		return true;
	}

	FOpenExrHeaderReader HeaderReader(ImagePath);
	if (!HeaderReader.HasInputFile())
	{
		return false;
	}

	// Get all channel names from the file.
	TArray<FString> ChannelNames;
	HeaderReader.GetChannelNames(ChannelNames);

	if (ChannelNames.Num() == 0)
	{
		return false;
	}

	// Parse channels into a layer map: layer prefix -> array of suffixes.
	// Channel "R" -> layer "", suffix "R"
	// Channel "beauty.R" -> layer "beauty", suffix "R"
	// Channel "Zdepth.Z" -> layer "Zdepth", suffix "Z"
	TMap<FString, TArray<FString>> LayerMap;
	for (const FString& ChannelName : ChannelNames)
	{
		FString Prefix;
		FString Suffix;
		if (ChannelName.Split(TEXT("."), &Prefix, &Suffix, ESearchCase::CaseSensitive, ESearchDir::FromEnd))
		{
			LayerMap.FindOrAdd(Prefix).Add(Suffix);
		}
		else
		{
			// Un-prefixed channel (default layer).
			LayerMap.FindOrAdd(FString()).Add(ChannelName);
		}
	}

	// Match the configured LayerName against available layers.
	FString MatchedLayer;
	bool bFound = false;

	if (LayerName.IsEmpty())
	{
		// Empty = default layer (un-prefixed channels).
		if (LayerMap.Contains(FString()))
		{
			MatchedLayer = FString();
			bFound = true;
		}
	}
	else if (LayerName.Contains(TEXT("*")) || LayerName.Contains(TEXT("?")))
	{
		// Wildcard matching - sort keys for deterministic first-alphabetical match.
		TArray<FString> SortedKeys;
		LayerMap.GetKeys(SortedKeys);
		SortedKeys.Sort();

		TArray<FString> AllMatches;
		for (const FString& Key : SortedKeys)
		{
			if (!Key.IsEmpty() && Key.MatchesWildcard(LayerName))
			{
				AllMatches.Add(Key);
			}
		}

		if (AllMatches.Num() > 1)
		{
			UE_LOGFMT(LogImgMedia, Warning, "EXR layer wildcard '{0}' matched {1} layers in {2}; using first alphabetical match '{3}'",
				LayerName, AllMatches.Num(), ImagePath, AllMatches[0]);
		}

		if (AllMatches.Num() > 0)
		{
			MatchedLayer = AllMatches[0];
			bFound = true;
		}
	}
	else
	{
		// Exact match.
		if (LayerMap.Contains(LayerName))
		{
			MatchedLayer = LayerName;
			bFound = true;
		}
	}

	if (!bFound)
	{
		UE_LOGFMT(LogImgMedia, Error, "EXR layer '{0}' not found in {1}. Available layers (set LayerName to the quoted value, or leave empty for the default):", LayerName, ImagePath);
		for (const TPair<FString, TArray<FString>>& Pair : LayerMap)
		{
			UE_LOGFMT(LogImgMedia, Warning, "  '{0}' with channels: {1}{2}",
				Pair.Key,
				FString::Join(Pair.Value, TEXT(", ")),
				Pair.Key.IsEmpty() ? FString(TEXT(" (default)")) : FString());
		}
		// Fall back to default layer.
		if (LayerMap.Contains(FString()))
		{
			MatchedLayer = FString();
			bFound = true;
		}
		else
		{
			return false;
		}
	}

	// Get the suffixes for the matched layer.
	const TArray<FString>& Suffixes = LayerMap[MatchedLayer];

	// Build full channel name from layer prefix and suffix.
	auto MakeFullChannelName = [&MatchedLayer](const FString& Suffix) -> FString
	{
		if (MatchedLayer.IsEmpty())
		{
			return Suffix;
		}
		return MatchedLayer + TEXT(".") + Suffix;
	};

	// Map suffixes to RGBA output slots.
	ResolvedChannelR = FString();
	ResolvedChannelG = FString();
	ResolvedChannelB = FString();
	ResolvedChannelA = FString();

	// Check for standard RGBA suffixes.
	bool bHasR = Suffixes.Contains(TEXT("R"));
	bool bHasG = Suffixes.Contains(TEXT("G"));
	bool bHasB = Suffixes.Contains(TEXT("B"));
	bool bHasA = Suffixes.Contains(TEXT("A"));

	if (bHasR || bHasG || bHasB)
	{
		// Standard RGB(A) layer.
		if (bHasR) ResolvedChannelR = MakeFullChannelName(TEXT("R"));
		if (bHasG) ResolvedChannelG = MakeFullChannelName(TEXT("G"));
		if (bHasB) ResolvedChannelB = MakeFullChannelName(TEXT("B"));
		if (bHasA) ResolvedChannelA = MakeFullChannelName(TEXT("A"));
	}
	else if (Suffixes.Num() == 1)
	{
		// Single non-standard channel (e.g., Z) -> grayscale.
		FString FullName = MakeFullChannelName(Suffixes[0]);
		ResolvedChannelR = FullName;
		ResolvedChannelG = FullName;
		ResolvedChannelB = FullName;
		// Alpha stays empty (fill with 1.0).
	}
	else
	{
		// Multiple non-standard suffixes -> map to RGBA in order.
		if (Suffixes.Num() > 0) ResolvedChannelR = MakeFullChannelName(Suffixes[0]);
		if (Suffixes.Num() > 1) ResolvedChannelG = MakeFullChannelName(Suffixes[1]);
		if (Suffixes.Num() > 2) ResolvedChannelB = MakeFullChannelName(Suffixes[2]);
		if (Suffixes.Num() > 3) ResolvedChannelA = MakeFullChannelName(Suffixes[3]);
	}

	UE_LOGFMT(LogImgMedia, Log, "EXR layer resolved: '{0}' -> R='{1}', G='{2}', B='{3}', A='{4}'",
		LayerName.IsEmpty() ? FString(TEXT("(default)")) : LayerName,
		ResolvedChannelR, ResolvedChannelG, ResolvedChannelB, ResolvedChannelA);

	bLayerResolved.store(true, std::memory_order_release);
	return true;
}

/* FExrImgMediaReader implementation
 *****************************************************************************/

bool FExrImgMediaReader::GetInfo(const FString& FilePath, FImgMediaFrameInfo& OutInfo, const TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>& OutFrame)
{
	FOpenExrHeaderReader HeaderReader(FilePath);
	if (HeaderReader.HasInputFile() == false)
	{
		return false;
	}

	OutInfo.CompressionName = HeaderReader.GetCompressionName();
	OutInfo.Dim = HeaderReader.GetDataWindow();
	OutInfo.FrameRate = HeaderReader.GetFrameRate(ImgMedia::DefaultFrameRate);
	OutInfo.Srgb = false;
	OutInfo.UncompressedSize = HeaderReader.GetUncompressedSize();
	OutInfo.NumChannels = HeaderReader.GetNumChannels();
	OutInfo.NumBytesPerPixel = sizeof(uint16) * OutInfo.NumChannels;
	OutInfo.NumMipLevels = 1;

	int32 CustomFormat = 0;
	HeaderReader.GetIntAttribute(IImgMediaModule::CustomFormatAttributeName.Resolve().ToString(), CustomFormat);
	bool bIsCustomFormat = CustomFormat > 0;

	if (bIsCustomFormat)
	{
		OutInfo.FormatName = TEXT("EXR CUSTOM");
	}
	else
	{
		// Can GPU reader be utilized to read this EXR file.
		if (HeaderReader.IsOptimizedForGpu())
		{
			OutInfo.FormatName = TEXT("EXR GPU");
		}
		else
		{
			OutInfo.FormatName = TEXT("EXR");
		}

		OutInfo.bHasTiles = HeaderReader.GetTileSize(OutInfo.TileDimensions);
		OutInfo.TileBorder = 0;
	}

	if (OutInfo.bHasTiles)
	{
		OutInfo.NumTiles =
		{FMath::CeilToInt(float(OutInfo.Dim.X) / (OutInfo.TileDimensions.X + OutInfo.TileBorder * 2))
		, FMath::CeilToInt(float(OutInfo.Dim.Y) / (OutInfo.TileDimensions.Y + OutInfo.TileBorder * 2))};
		if (HeaderReader.ContainsMips())
		{
			OutInfo.NumMipLevels = HeaderReader.GetNumMipLevels();

			SIZE_T SizeMip0 = OutInfo.UncompressedSize;
			for (int32 Level = 1; Level < OutInfo.NumMipLevels; Level++)
			{
				OutInfo.UncompressedSize += HeaderReader.GetUncompressedMipSize(Level);
			}
		}
	}
	else
	{
		OutInfo.TileDimensions = OutInfo.Dim;
		OutInfo.NumTiles = FIntPoint(1, 1);
	}

	if (OutFrame.IsValid())
	{
		OutFrame->SetInfo(OutInfo);
	}

	return (OutInfo.UncompressedSize > 0) && (OutInfo.Dim.GetMin() > 0);
}

SIZE_T FExrImgMediaReader::GetMipBufferTotalSize(FIntPoint Dim, bool bInHasMips)
{
	SIZE_T Size = 0;
	if (bInHasMips)
	{
		Size = ((Dim.X * Dim.Y * 4) / 3) * sizeof(uint16) * 4;
	}
	else
	{
		Size = (Dim.X * Dim.Y ) * sizeof(uint16) * 4;
	}

	return Size;
}

#endif //IMGMEDIA_EXR_SUPPORTED_PLATFORM
