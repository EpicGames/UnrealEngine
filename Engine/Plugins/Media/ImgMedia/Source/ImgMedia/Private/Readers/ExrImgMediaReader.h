// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImgMediaPrivate.h"

#include <atomic>

#if IMGMEDIA_EXR_SUPPORTED_PLATFORM

#include "IImgMediaReader.h"
#include "ExrReaderGpu.h"

class FImgMediaLoader;
class FOpenExrHeaderReader;
class FRgbaInputFile;
struct FSampleConverterParameters;

/**
 * Implements a reader for EXR image sequences.
 */
class FExrImgMediaReader
	: public IImgMediaReader
{
public:

	/** Constructor. */
	FExrImgMediaReader(const TSharedRef<FImgMediaLoader, ESPMode::ThreadSafe>& InLoader, const FString& InLayerName = FString());
	virtual ~FExrImgMediaReader();
public:

	//~ IImgMediaReader interface

	virtual bool GetFrameInfo(const FString& ImagePath, FImgMediaFrameInfo& OutInfo) override;
	virtual bool ReadFrame(int32 FrameId, const TMap<int32, FMediaTileSelection>& InMipTiles, TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> OutFrame) override;
	virtual void CancelFrame(int32 FrameNumber) override;
	virtual void UncancelFrame(int32 FrameNumber) override;

public:
	/** Gets reader type (GPU vs CPU) depending on size of EXR and its compression. */
	static TSharedPtr<IImgMediaReader, ESPMode::ThreadSafe> GetReader(const TSharedRef<FImgMediaLoader, ESPMode::ThreadSafe>& InLoader, FString FirstImageInSequencePath, const FString& InLayerName = FString());

protected:
	enum EReadResult
	{
		Fail,
		Success,
		Cancelled,
		Skipped
	};

	/**
	 * Get the frame information from the given input file.
	 *
	 * @param FilePath The location of the exr file.
	 * @param OutInfo Will contain the frame information.
	 * @param OutFrame Optional frame to update.
	 * @return true on success, false otherwise.
	 */
	static bool GetInfo(const FString& FilePath, FImgMediaFrameInfo& OutInfo, const TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>& OutFrame = nullptr);

	/**
	 * Reads tiles from exr files in tile rows based on Tile region. If frame is pending for cancelation
	 * stops reading tiles at the current tile row.
	 * 
	 * @param Buffer					A buffer to read into from hard drive.
	 * @param BufferSize				The total size of the buffer. Used to limit reader to buffer region.
	 * @param ImagePath					Path to the file to read.
	 * @param FrameId					Frame id used to determine if frame has been canceled.
	 * @param TileRegions				Rectangular tile regions that we read from the hard drive line by line.
	 * @param ConverterParams			Full information about the current frame.
	 * @param OutBufferRegionsToCopy	This buffer is used to issue copy commands on GPU Copy thread.
	*/
	EReadResult ReadTiles
		( uint16* Buffer
		, int64 BufferSize
		, const FString& ImagePath
		, const TArray<FIntRect>& TileRegions
		, FSampleConverterParameters& ConverterParams
		, const int32 CurrentMipLevel
		, TArray<UE::Math::TIntPoint<int64>>& OutBufferRegionsToCopy);

	/**
	 * Gets the total size needed for all mips.
	 *
	 * @param Dim Dimensions of the largest mip.
	 * @return Total size of all mip levels.
	 */
	SIZE_T GetMipBufferTotalSize(FIntPoint Dim, bool bInHasMips);

	/**
	 * Resolves the configured LayerName to concrete RGBA channel names.
	 * Reads the channel list from the given file and populates ResolvedChannel[R/G/B/A].
	 *
	 * @param ImagePath Path to an EXR file to read channel names from.
	 * @return true if resolution succeeded.
	 */
	bool ResolveLayer(const FString& ImagePath);

protected:
	TSet<int32> CanceledFrames;
	FCriticalSection CanceledFramesCriticalSection;
	TMap<int32, FRgbaInputFile*> PendingFrames;
	TWeakPtr<FImgMediaLoader, ESPMode::ThreadSafe> LoaderPtr;

	/** Configured layer name (may contain wildcards). */
	FString LayerName;
	/** Resolved channel names for each RGBA output slot. */
	FString ResolvedChannelR;
	FString ResolvedChannelG;
	FString ResolvedChannelB;
	FString ResolvedChannelA;
	/** Guards layer resolution state (separate from CanceledFramesCriticalSection). */
	FCriticalSection LayerResolveCriticalSection;
	/** True after layer has been resolved to concrete channel names. */
	std::atomic_bool bLayerResolved{false};
};


#endif //IMGMEDIA_EXR_SUPPORTED_PLATFORM
