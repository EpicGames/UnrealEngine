// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExrImgMediaReaderGpu.h"

#if defined(PLATFORM_WINDOWS) && PLATFORM_WINDOWS

#include "Async/Async.h"
#include "Misc/Paths.h"
#include "OpenExrWrapper.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"
#include "ExrReaderGpu.h"
#include "RHICommandList.h"
#include "ExrSwizzlingShader.h"
#include "MediaShaders.h"
#include "SceneUtils.h"


#include "Engine/TextureRenderTarget2D.h"
#include "GlobalShader.h"
#include "Logging/StructuredLog.h"
#include "CommonRenderResources.h"
#include "BaseMediaSourceColorSettings.h"
#include "ColorManagement/TransferFunctions.h"
#include "Loader/ImgMediaLoader.h"
#include "PostProcess/DrawRectangle.h"
#include "RHIStaticStates.h"
#include "SceneUtils.h"
#include "ShaderParameterUtils.h"
#include "TextureResource.h"
#include "ScreenPass.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"


DECLARE_GPU_STAT_NAMED(ExrImgMediaReaderGpu, TEXT("ExrImgGpu"));
DECLARE_GPU_STAT_NAMED(ExrImgMediaReaderGpu_MipRender, TEXT("ExrImgGpu.MipRender"));
DECLARE_GPU_STAT_NAMED(ExrImgMediaReaderGpu_MipUpscale, TEXT("ExrImgGpu.MipRender.MipUpscale"));
DECLARE_GPU_STAT_NAMED(ExrImgMediaReaderGpu_CopyUploadBuffer, TEXT("ExrImgGpu.MipRender.UploadBufferCopy"));
DECLARE_GPU_STAT_NAMED(ExrImgMediaReaderGpu_AllocateBuffer, TEXT("ExrImgGpu.MipRender.AllocateBuffer"));

static bool bExrReaderUseUploadHeap = true;
static FAutoConsoleVariableRef CVarExrReaderUseUploadHeap(
	TEXT("r.ExrReaderGPU.UseUploadHeap"),
	bExrReaderUseUploadHeap,
	TEXT("Utilizes upload heap and copies raw exr buffer asynchronously.\n")
	TEXT("Read-only and to be set in a config file (requires restart)."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

namespace {

	/** This function is similar to DrawScreenPass in OpenColorIODisplayExtension.cpp except it is catered for Viewless texture rendering. */
	template<typename TSetupFunction>
	void DrawScreenPass(
		FRHICommandListImmediate& RHICmdList,
		const FIntPoint& OutputResolution,
		const FIntRect& Viewport,
		const FScreenPassPipelineState& PipelineState,
		TSetupFunction SetupFunction)
	{
		RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.f, Viewport.Max.X, Viewport.Max.Y, 1.0f);

		SetScreenPassPipelineState(RHICmdList, PipelineState);

		// Setting up buffers.
		SetupFunction(RHICmdList);

		EDrawRectangleFlags DrawRectangleFlags = EDRF_UseTriangleOptimization;

		UE::Renderer::PostProcess::DrawPostProcessPass(
			RHICmdList,
			PipelineState.VertexShader,
			0, 0, OutputResolution.X, OutputResolution.Y,
			Viewport.Min.X, Viewport.Min.Y, Viewport.Width(), Viewport.Height(),
			OutputResolution,
			OutputResolution,
			INDEX_NONE,
			false,
			DrawRectangleFlags);
	}
}


/* FExrImgMediaReaderGpu structors
 *****************************************************************************/

FExrImgMediaReaderGpu::FExrImgMediaReaderGpu(const TSharedRef<FImgMediaLoader, ESPMode::ThreadSafe>& InLoader, const FString& InLayerName)
	: FExrImgMediaReader(InLoader, InLayerName)
	, LastTickedFrameCounter((uint64)-1)
	, bIsShuttingDown(false)
	, bFallBackToCPU(false)
{
}


FExrImgMediaReaderGpu::~FExrImgMediaReaderGpu()
{
	FScopeLock ScopeLock(&MemoryPoolCriticalSection);

	// Copy memory pool array to be released on render thread.
	ENQUEUE_RENDER_COMMAND(DeletePooledBuffers)([InMemoryPool = MemoryPool](FRHICommandListImmediate& RHICmdList)
	{
		SCOPED_DRAW_EVENT(RHICmdList, FExrImgMediaReaderGpu_ReleaseMemoryPool);
		TArray<uint32> KeysForIteration;
		InMemoryPool.GetKeys(KeysForIteration);
		for (uint32 Key : KeysForIteration)
		{
			TArray<FStructuredBufferPoolItem*> AllValues;
			InMemoryPool.MultiFind(Key, AllValues);
			for (FStructuredBufferPoolItem* MemoryPoolItem : AllValues)
			{
				delete MemoryPoolItem;
			}
		}
	});
}

bool FExrImgMediaReaderGpu::GetFrameInfo(const FString& ImagePath, FImgMediaFrameInfo& OutInfo)
{
	// Skip the CPU reader's single-mip clamp; the GPU path reads in-file mip levels natively.
	return GetInfo(ImagePath, OutInfo);
}

FExrImgMediaReader::EReadResult FExrImgMediaReaderGpu::ReadMip
	( const int32 CurrentMipLevel
	, const FMediaTileSelection& CurrentTileSelection
	, TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> OutFrame
	, FSampleConverterParameters& ConverterParams
	, TSharedPtr<FExrMediaTextureSampleConverter, ESPMode::ThreadSafe> SampleConverter
	, const FString& ImagePath)
{

	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("ExrReaderGpu.ReadMip %d"), CurrentMipLevel));

	// Next mip level.
	const int MipLevelDiv = 1 << CurrentMipLevel;
	const FIntPoint CurrentMipDim = (ConverterParams.FullResolution / MipLevelDiv).ComponentMax(FIntPoint(1,1));
	const FImgMediaFrameInfo& FrameInfo = ConverterParams.FrameInfo;
	const FIntPoint CurrentNumTiles = FIntPoint(
		FMath::Max(1, FMath::CeilToInt(static_cast<float>(FrameInfo.NumTiles.X) / MipLevelDiv)),
		FMath::Max(1, FMath::CeilToInt(static_cast<float>(FrameInfo.NumTiles.Y) / MipLevelDiv)));

	const SIZE_T BufferSize = GetBufferSize(CurrentMipDim, FrameInfo.NumChannels, FrameInfo.bHasTiles, CurrentNumTiles);
	
	FStructuredBufferPoolItemSharedPtr BufferData = SampleConverter->GetOrCreateMipLevelBuffer(
		CurrentMipLevel,
		[this, BufferSize]() -> FStructuredBufferPoolItemSharedPtr
		{
			return AllocateGpuBufferFromPool(BufferSize);
		}
	);

	uint16* MipDataPtr = static_cast<uint16*>(BufferData->UploadBufferMapped);

	EReadResult ReadResult = Fail;

	if (FPaths::FileExists(ImagePath))
	{
		TArray<UE::Math::TIntPoint<int64>> BufferRegionsToCopy;
		// read frame data
		if (FrameInfo.bHasTiles)
		{
			TArray<FIntRect> TileRegionsToRead;
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("ExrReaderGpu.CalculateRegions %d"), CurrentMipLevel));

				if (!OutFrame->MipTilesPresent.GetVisibleRegions(CurrentMipLevel, CurrentTileSelection, TileRegionsToRead))
				{
					TileRegionsToRead = CurrentTileSelection.GetVisibleRegions();
				}
			}

			if (TileRegionsToRead.IsEmpty() && CurrentTileSelection.IsAnyVisible())
			{
				// If all tiles were previously read and stored in cached frame, reading can be skipped.
				ReadResult = Skipped;
			}
			else
			{
				ReadResult = ReadTiles(MipDataPtr, BufferSize, ImagePath, TileRegionsToRead, ConverterParams, CurrentMipLevel, BufferRegionsToCopy);
				
				for (const FIntRect& Region : TileRegionsToRead)
				{
					OutFrame->NumTilesRead += Region.Area();
				}
			}
		}
		else
		{
			ReadResult = ReadInChunks(MipDataPtr, ImagePath, ConverterParams.FrameId, CurrentMipDim, BufferSize);
			OutFrame->NumTilesRead++;
		}

		if (ReadResult == Success && bExrReaderUseUploadHeap)
		{
			ENQUEUE_RENDER_COMMAND(CopyFromUploadBuffer)([SampleConverter, BufferData, BufferRegionsToCopy, FrameId = ConverterParams.FrameId](FRHICommandListImmediate& RHICmdList)
				{
					RHI_BREADCRUMB_EVENT_STAT_F(RHICmdList, ExrImgMediaReaderGpu_CopyUploadBuffer, "ExrReaderGpu.StartCopy", "ExrReaderGpu.StartCopy %d", FrameId);

					RHICmdList.Transition(FRHITransitionInfo(BufferData->ShaderAccessBufferRef, ERHIAccess::SRVMask, ERHIAccess::CopyDest));

					if (BufferRegionsToCopy.IsEmpty())
					{
						RHICmdList.CopyBufferRegion(BufferData->ShaderAccessBufferRef, 0, BufferData->UploadBufferRef, 0, BufferData->ShaderAccessBufferRef->GetSize());
					}
					else
					{
						for (UE::Math::TIntPoint<int64> Region : BufferRegionsToCopy)
						{
							RHICmdList.CopyBufferRegion(BufferData->ShaderAccessBufferRef, Region.X, BufferData->UploadBufferRef, Region.X, Region.Y);
						}
					}

					RHICmdList.Transition(FRHITransitionInfo(BufferData->ShaderAccessBufferRef, ERHIAccess::CopyDest, ERHIAccess::SRVMask));
				});
		}
	}
	else
	{
		UE_LOGF(LogImgMedia, Error, "Could not load %ls", *ImagePath);
		return Fail;
	}

	return ReadResult;
}

/* FExrImgMediaReaderGpu interface
 *****************************************************************************/

bool FExrImgMediaReaderGpu::ReadFrame(int32 FrameId, const TMap<int32, FMediaTileSelection>& InMipTiles, TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> OutFrame)
{
	// Fall back to cpu?
	if (bFallBackToCPU.load(std::memory_order_acquire))
	{
		return FExrImgMediaReader::ReadFrame(FrameId, InMipTiles, OutFrame);
	}

	TSharedPtr<FImgMediaLoader, ESPMode::ThreadSafe> Loader = LoaderPtr.Pin();
	if (Loader.IsValid() == false)
	{
		return false;
	}

	/**
	 * The GPU raw block reader cannot perform per-channel selection. Any file
	 * that warrants a LayerName almost certainly has more than four channels
	 * total, which IsOptimizedForGpu() would already have rejected. Delegate
	 * all layer-name reads to the CPU reader which handles them correctly.
	 */
	if (!LayerName.IsEmpty())
	{
		UE_LOGFMT(LogImgMedia, Warning,
			"GPU reader does not support EXR layer selection (layer '{0}'). Falling back to CPU reader.",
			LayerName);
		bFallBackToCPU.store(true, std::memory_order_release);
		
		return FExrImgMediaReader::ReadFrame(FrameId, InMipTiles, OutFrame);
	}

	const FString& LargestImagePath = Loader->GetImagePath(FrameId, 0);
	FImgMediaFrameInfo FrameInfo;
	if (!GetInfo(LargestImagePath, FrameInfo, OutFrame))
	{
		return false;
	}

	TSharedPtr<FExrMediaTextureSampleConverter, ESPMode::ThreadSafe> SampleConverterPtr = OutFrame->GetOrCreateSampleConverter<FExrMediaTextureSampleConverter>();

	// Get tile info.
	FSampleConverterParameters ConverterParams = SampleConverterPtr->GetParams();
	ConverterParams.FullResolution = FrameInfo.Dim;
	ConverterParams.FrameId = FrameId;
	if (ConverterParams.FullResolution.GetMin() <= 0)
	{
		return false;
	}

	ConverterParams.FrameInfo = FrameInfo;
	ConverterParams.PixelSize = sizeof(uint16) * ConverterParams.FrameInfo.NumChannels;
	ConverterParams.TileDimWithBorders = FrameInfo.TileDimensions + FrameInfo.TileBorder * 2;
	ConverterParams.NumMipLevels = Loader->GetNumMipLevels();
	ConverterParams.bMipsInSeparateFiles = Loader->MipsInSeparateFiles();
	ConverterParams.SourceColorSettings = Loader->GetSourceColorSettings();

	{
		// Force mip level to be upscaled to all higher quality mips.
		TMap<int32, FMediaTileSelection> InMipTilesCopy = InMipTiles;
		const int32 MipToUpscale = FMath::Clamp(Loader->GetMinimumLevelToUpscale(), -1, ConverterParams.NumMipLevels - 1);

		if (ConverterParams.NumMipLevels > 1 && MipToUpscale >= 0)
		{
			ConverterParams.UpscaleMip = MipToUpscale;

			FMediaTileSelection FullSelection = FMediaTileSelection::CreateForTargetMipLevel(ConverterParams.FullResolution, FrameInfo.TileDimensions, MipToUpscale, true);
			if (InMipTilesCopy.Contains(MipToUpscale))
			{
				InMipTilesCopy[MipToUpscale] = FullSelection;
			}
			else
			{
				InMipTilesCopy.Add(MipToUpscale, FullSelection);
			}
		}

		// Loop over all mips.
		for (const TPair<int32, FMediaTileSelection>& TilesPerMip : InMipTilesCopy)
		{
			const int32 CurrentMipLevel = TilesPerMip.Key;
			const FMediaTileSelection& CurrentTileSelection = TilesPerMip.Value;

			if (!CurrentTileSelection.IsAnyVisible())
			{
				continue;
			}

			// Get highest resolution mip level path.
			FString ImagePath = Loader->GetImagePath(ConverterParams.FrameId, ConverterParams.bMipsInSeparateFiles ? CurrentMipLevel : 0);

			EReadResult ReadResult = ReadMip(CurrentMipLevel, CurrentTileSelection, OutFrame, ConverterParams, SampleConverterPtr, ImagePath);
			switch (ReadResult)
			{
			case FExrImgMediaReader::Success:
				OutFrame->MipTilesPresent.Include(CurrentMipLevel, CurrentTileSelection);
				break;
			case FExrImgMediaReader::Fail:
				{
					// Check if we have a compressed file.
					FImgMediaFrameInfo Info;
					if (GetInfo(ImagePath, Info))
					{
						if (Info.CompressionName != "Uncompressed")
						{
							UE_LOGF(LogImgMedia, Error, "GPU Reader cannot read compressed file %ls.", *ImagePath);
							UE_LOGF(LogImgMedia, Error, "Compressed and uncompressed files should not be mixed in a single sequence.");
						}
					}

					// Fall back to CPU.
					bFallBackToCPU.store(true, std::memory_order_release);

					return FExrImgMediaReader::ReadFrame(ConverterParams.FrameId, InMipTiles, OutFrame);
				}
				break;
			case FExrImgMediaReader::Cancelled:
				// Abort further reading
				return false;
			case FExrImgMediaReader::Skipped:
				// No new tiles were read, continue to the next mip level.
				break;

			default:
				checkNoEntry();
				break;
			};
		}
		
		// Create viewport(s) with all mip/tiles present
		FScopeLock Lock(&OutFrame->MipTilesPresent.CriticalSection);
		for (const TPair<int32, FMediaTileSelection>& TilesPerMip : OutFrame->MipTilesPresent.GetDataUnsafe())
		{
			const FMediaTileSelection& CurrentTileSelection = TilesPerMip.Value;
			const int32 CurrentMipLevel = TilesPerMip.Key;

			// Skip this viewport since we don't have anything to render.
			if (!SampleConverterPtr->HasMipLevelBuffer(CurrentMipLevel))
			{
				continue;
			}

			const int32 MipLevelDiv = 1 << CurrentMipLevel;
			const FIntPoint CurrentMipDim = (ConverterParams.FullResolution / MipLevelDiv).ComponentMax(FIntPoint(1,1));

			TArray<FIntRect>& Viewports = ConverterParams.Viewports.Add(CurrentMipLevel);
			for (const FIntRect& TileRegion : CurrentTileSelection.GetVisibleRegions())
			{
				FIntRect Viewport;
				if (ConverterParams.FrameInfo.bHasTiles)
				{
					Viewport.Min = FIntPoint(ConverterParams.TileDimWithBorders.X * TileRegion.Min.X, ConverterParams.TileDimWithBorders.Y * TileRegion.Min.Y);
					Viewport.Max = FIntPoint(ConverterParams.TileDimWithBorders.X * TileRegion.Max.X, ConverterParams.TileDimWithBorders.Y * TileRegion.Max.Y);
					Viewport.Clip(FIntRect(FIntPoint::ZeroValue, CurrentMipDim));
				}
				else
				{
					Viewport.Min = FIntPoint(0, 0);
					Viewport.Max = CurrentMipDim;
				}
				Viewports.Add(Viewport);
			}
		}
	}

	OutFrame->Format = ConverterParams.FrameInfo.NumChannels <= 3 ? EMediaTextureSampleFormat::FloatRGB : EMediaTextureSampleFormat::FloatRGBA;
	OutFrame->Stride = ConverterParams.FullResolution.X * ConverterParams.PixelSize;
	
	SampleConverterPtr->SetParams(ConverterParams);
	
	CreateSampleConverterCallback(SampleConverterPtr);

	UE_LOGF(LogImgMedia, Verbose, "Reader %p: Read Pixels Complete. %i", this, FrameId);
	return true;
}

void FExrImgMediaReaderGpu::PreAllocateMemoryPool(int32 NumFrames, const FImgMediaFrameInfo& FrameInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("ExrReaderGpu.PreAllocateMemoryPool");
	SIZE_T AllocSize = GetBufferSize(FrameInfo.Dim, FrameInfo.NumChannels, FrameInfo.bHasTiles, FrameInfo.NumTiles);
	for (int32 FrameCacheNum = 0; FrameCacheNum < NumFrames; FrameCacheNum++)
	{
		AllocateGpuBufferFromPool(AllocSize);
	}
}

/* FExrImgMediaReaderGpu implementation
 *****************************************************************************/

FExrImgMediaReaderGpu::EReadResult FExrImgMediaReaderGpu::ReadInChunks(uint16* Buffer, const FString& ImagePath, int32 FrameId, const FIntPoint& Dim, int32 BufferSize)
{
	EReadResult bResult = Success;

	// Chunks are of 16 MB
	const int32 ChunkSize = 0xF42400;
	const int32 Remainder = BufferSize % ChunkSize;
	const int32 NumChunks = (BufferSize - Remainder) / ChunkSize;
	int32 CurrentBufferPos = 0;
	FExrReader ChunkReader;

	// Since ReadInChunks is only utilized for exr files without tiles and mips, Num Mip levels is always 1.
	const int32 NumLevels = 1;
	TArray<int32> NumTOffsetsPerLevel;
	NumTOffsetsPerLevel.Add(Dim.Y);
	if (!ChunkReader.OpenExrAndPrepareForPixelReading(ImagePath, NumTOffsetsPerLevel))
	{
		return Fail;
	}

	for (int32 Row = 0; Row <= NumChunks; Row++)
	{
		int32 Step = Row == NumChunks ? Remainder : ChunkSize;
		if (Step == 0)
		{
			break;
		}

		// Check to see if the frame was canceled.
		{
			FScopeLock RegionScopeLock(&CanceledFramesCriticalSection);
			if (CanceledFrames.Remove(FrameId) > 0)
			{
				UE_LOGF(LogImgMedia, Verbose, "Reader %p: Canceling Frame %i At chunk # %i", this, FrameId, Row);
				bResult = Cancelled;
				break;
			}
		}

		if (!ChunkReader.ReadExrImageChunk(reinterpret_cast<char*>(Buffer) + CurrentBufferPos, Step))
		{
			bResult = Fail;
			break;
		}
		CurrentBufferPos += Step;
	}

	if (!ChunkReader.CloseExrFile())
	{
		return Fail;
	}

	return bResult;
}

SIZE_T FExrImgMediaReaderGpu::GetBufferSize(const FIntPoint& Dim, int32 NumChannels, bool bHasTiles, const FIntPoint& TileNum)
{
	if (!bHasTiles)
	{
		/** 
		* Reading scanlines.
		* 
		* At the beginning of each row of B G R channel planes there is 2x4 byte data that has information
		* about number of pixels in the current row and row's number.
		*/
		const uint16 Padding = FExrReader::PLANAR_RGB_SCANLINE_PADDING;
		SIZE_T BufferSize = Dim.X * Dim.Y * sizeof(uint16) * NumChannels + Dim.Y * Padding;
		return BufferSize;
	}
	else
	{
		/** 
		* Reading tiles.
		* 
		* At the beginning of each tile there is 20 byte data that has information
		* about number contents of tiles.
		*/
		const uint16 Padding = FExrReader::TILE_PADDING;
		SIZE_T BufferSize = Dim.X * Dim.Y * sizeof(uint16) * NumChannels + (TileNum.X * TileNum.Y) * Padding;
		return BufferSize;
	}

}

void FExrImgMediaReaderGpu::CreateSampleConverterCallback(TSharedPtr<FExrMediaTextureSampleConverter, ESPMode::ThreadSafe> SampleConverter)
{
	auto RenderThreadSwizzler = [] (FRHICommandListImmediate& RHICmdList, FTextureRHIRef RenderTargetTextureRHI, TMap<int32, FStructuredBufferPoolItemSharedPtr>& MipBuffers, const FSampleConverterParameters ConverterParams)->bool
	{
		RHI_BREADCRUMB_EVENT_STAT_F(RHICmdList, ExrImgMediaReaderGpu, "ExrReaderGpu.Convert", "ExrReaderGpu.Convert %d", ConverterParams.FrameId);

		auto RenderMip = []
			( FRHICommandListImmediate& RHICmdList
			, FTextureRHIRef RenderTargetTextureRHI
			, const FSampleConverterParameters & ConverterParams
			, int32 SampleMipLevel
			, int32 TextureMipLevel
			, FStructuredBufferPoolItemSharedPtr BufferData
			, const FIntPoint& SampleSize
			, const FIntPoint& TextureSize
			, const TArray<FIntRect>& MipViewports)
		{
			RHI_BREADCRUMB_EVENT_STAT(RHICmdList, ExrImgMediaReaderGpu_MipRender, "ExrImgGpu.MipRender");

			FRHIRenderPassInfo RPInfo(RenderTargetTextureRHI, ERenderTargetActions::DontLoad_Store, nullptr, TextureMipLevel);
			RHICmdList.BeginRenderPass(RPInfo, TEXT("ExrTextureSwizzle"));

			FExrSwizzlePS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FExrSwizzlePS::FRgbaSwizzle>(ConverterParams.FrameInfo.NumChannels - 1);
			PermutationVector.Set<FExrSwizzlePS::FRenderTiles>(ConverterParams.FrameInfo.bHasTiles);
			PermutationVector.Set<FExrSwizzlePS::FPartialTiles>(false);

			FExrSwizzlePS::FParameters Parameters = FExrSwizzlePS::FParameters();
			Parameters.TextureSize = SampleSize;
			Parameters.TileSize = ConverterParams.TileDimWithBorders;
			Parameters.NumChannels = ConverterParams.FrameInfo.NumChannels;
			if (ConverterParams.FrameInfo.bHasTiles)
			{
				Parameters.NumTiles = FIntPoint(FMath::CeilToInt(float(SampleSize.X) / ConverterParams.TileDimWithBorders.X), FMath::CeilToInt(float(SampleSize.Y) / ConverterParams.TileDimWithBorders.Y));
			}
			if (ConverterParams.SourceColorSettings.IsValid())
			{
				Parameters.bApplyColorTransform = 1u;
				const UE::Color::EEncoding EncodingOverride = ConverterParams.SourceColorSettings->GetEncodingOverride();
				Parameters.EOTF = static_cast<uint32>(EncodingOverride);

				const UE::Color::FColorSpace& DestinationCS = UE::Color::FColorSpace::GetWorking();
				const UE::Color::FColorSpace& SourceCS = ConverterParams.SourceColorSettings->GetColorSpaceOverride(DestinationCS);

				if (SourceCS.Equals(DestinationCS))
				{
					Parameters.ColorSpaceMatrix = FMatrix44f::Identity;
				}
				else
				{
					const UE::Color::EChromaticAdaptationMethod Method = ConverterParams.SourceColorSettings->GetChromaticAdaptationMethod();
					Parameters.ColorSpaceMatrix = UE::Color::Transpose<float>(UE::Color::FColorSpaceTransform(SourceCS, DestinationCS, Method));
				}

				const UE::Color::EReferenceWhite ReferenceWhite = ConverterParams.SourceColorSettings->GetReferenceWhiteOverride();
				const float NormalizationFactor = UE::Color::GetReferenceWhiteLinearScale(EncodingOverride, ReferenceWhite);
				if (!FMath::IsNearlyEqual(NormalizationFactor, 1.0f))
				{
					Parameters.ColorSpaceMatrix = Parameters.ColorSpaceMatrix.ApplyScale(NormalizationFactor);
				}
			}
			else
			{
				Parameters.bApplyColorTransform = 0u;
				Parameters.EOTF = 0u;
				Parameters.ColorSpaceMatrix = FMatrix44f::Identity;
			}

			if (ConverterParams.FrameInfo.bHasTiles &&
				(ConverterParams.TileInfoPerMipLevel.Num() > SampleMipLevel && ConverterParams.TileInfoPerMipLevel[SampleMipLevel].Num() > 0))
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_STR("ExrReaderGpu.TileDesc");

				// This buffer is allocated on already allocated block, therefore the risk of fragmentation is mitigated.
				const FRHIBufferCreateDesc CreateDesc =
					FRHIBufferCreateDesc::CreateStructured<FExrReader::FTileDesc>(TEXT("FExrImgMediaReaderGpu_TileDesc"), ConverterParams.TileInfoPerMipLevel[SampleMipLevel].Num())
					.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Dynamic | EBufferUsageFlags::FastVRAM)
					.SetInitActionInitializer()
					.DetermineInitialState();

				TRHIBufferInitializer<FExrReader::FTileDesc> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);
				Initializer.WriteArray(MakeConstArrayView(ConverterParams.TileInfoPerMipLevel[SampleMipLevel]));

				FBufferRHIRef BufferRef = Initializer.Finalize();

				Parameters.TileDescBuffer = RHICmdList.CreateShaderResourceView(BufferRef, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(BufferRef));
				PermutationVector.Set<FExrSwizzlePS::FPartialTiles>(true);
			}

			Parameters.UnswizzledBuffer = BufferData->ShaderResourceView;

			FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

			TShaderMapRef<FExrSwizzleVS> SwizzleShaderVS(ShaderMap);
			TShaderMapRef<FExrSwizzlePS> SwizzleShaderPS(ShaderMap, PermutationVector);

			FScreenPassPipelineState PipelineState(SwizzleShaderVS, SwizzleShaderPS, TStaticBlendState<>::GetRHI(), TStaticDepthStencilState<false, CF_Always>::GetRHI());

			// If there are tiles determines if we should deliver tiles one by one or in a bulk.
			for (const FIntRect& Viewport : MipViewports)
			{
				DrawScreenPass(RHICmdList, TextureSize, Viewport, PipelineState, [&](FRHICommandListImmediate& RHICmdList)
					{
						SetShaderParameters(RHICmdList, SwizzleShaderPS, SwizzleShaderPS.GetPixelShader(), Parameters);
					});
			}

			// Resolve render target.
			RHICmdList.EndRenderPass();
		};

		// Using the same transition pattern as ConvertTextureToOutput.
		RHICmdList.Transition(FRHITransitionInfo(RenderTargetTextureRHI, ERHIAccess::Unknown, ERHIAccess::RTV));

		int32 MipToUpscale = ConverterParams.UpscaleMip;

		// Upscale to all mips below the mip to upscale.
		for (int32 MipLevel = 0; MipLevel <= MipToUpscale; MipLevel++)
		{
			int MipLevelDiv = 1 << MipLevel;
			const FIntPoint Dim = (ConverterParams.FullResolution / MipLevelDiv).ComponentMax(FIntPoint(1,1));

			{
				RHI_BREADCRUMB_EVENT_STAT(RHICmdList, ExrImgMediaReaderGpu_MipUpscale, "ExrImgGpu.MipRender.MipUpscale");

				// Sanity check.
				if (!MipBuffers.Contains(MipToUpscale))
				{
					UE_LOGF(LogImgMedia, Warning, "Requested mip could not be found %d", MipToUpscale);
				}

				FStructuredBufferPoolItemSharedPtr BufferDataToUpscale = MipBuffers[MipToUpscale];
				TArray<FIntRect> FakeViewport;
				FakeViewport.Add(FIntRect(FIntPoint(0, 0), Dim));
				RenderMip(RHICmdList, RenderTargetTextureRHI, ConverterParams, MipToUpscale, MipLevel, BufferDataToUpscale, (ConverterParams.FullResolution / (1 << MipToUpscale)).ComponentMax(FIntPoint(1,1)), Dim, FakeViewport);
			}
		}

		for (const TPair<int32, TArray<FIntRect>>& MipLevelViewports : ConverterParams.Viewports)
		{
			int32 MipLevel = MipLevelViewports.Key;
			
			// Sanity check.
			if (!MipBuffers.Contains(MipLevel))
			{
				continue;
			}

			FStructuredBufferPoolItemSharedPtr BufferData = MipBuffers[MipLevel];
			int MipLevelDiv = 1 << MipLevel;
			const FIntPoint Dim = (ConverterParams.FullResolution / MipLevelDiv).ComponentMax(FIntPoint(1,1));

			if (BufferData.IsValid())
			{
				if (!BufferData->UploadBufferRef.IsValid() || (bExrReaderUseUploadHeap && !BufferData->ShaderAccessBufferRef->IsValid()))
				{
					continue;
				}

				// Skip the mip to upscale because it is read and rendered already.
				if (MipLevelViewports.Key == MipToUpscale)
				{
					continue;
				}
				RenderMip(RHICmdList, RenderTargetTextureRHI, ConverterParams, MipLevel, MipLevel, BufferData, Dim, Dim, MipLevelViewports.Value);
			}
		}

		RHICmdList.Transition(FRHITransitionInfo(RenderTargetTextureRHI, ERHIAccess::RTV, ERHIAccess::SRVMask));

		//Doesn't need further conversion so returning false.
		return false;
	};

	// Stacks up converters for each tile region.
	SampleConverter->AddCallback(FExrConvertBufferCallback::CreateLambda(RenderThreadSwizzler));
}

FStructuredBufferPoolItemSharedPtr FExrImgMediaReaderGpu::AllocateGpuBufferFromPool(uint32 AllocSize)
{
	//TRACE_CPUPROFILER_EVENT_SCOPE_STR("ExrReaderGpu.AllocBuffer");
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("ExrReaderGpu.AllocBuffer %d"), AllocSize));
	TWeakPtr<FExrImgMediaReaderGpu, ESPMode::ThreadSafe> WeakReaderPtr = AsWeak();

	// This function is attached to the shared pointer and is used to return any allocated memory to staging pool.
	auto BufferDeleter = [WeakReaderPtr, AllocSize](FStructuredBufferPoolItem* ObjectToDelete) {
		TSharedPtr<FExrImgMediaReaderGpu, ESPMode::ThreadSafe> SharedReaderPtr = WeakReaderPtr.Pin();
		if (SharedReaderPtr.IsValid())
		{
			SharedReaderPtr->ReturnGpuBufferToPool(AllocSize, ObjectToDelete);
		}
		else
		{
			ENQUEUE_RENDER_COMMAND(DeletePooledBuffers)([ObjectToDelete](FRHICommandListImmediate& RHICmdList)
			{
				delete ObjectToDelete;
			});
		}
	};

	// Buffer that ends up being returned out of this function.
	FStructuredBufferPoolItemSharedPtr AllocatedBuffer;

	{
		FScopeLock ScopeLock(&MemoryPoolCriticalSection);
		FStructuredBufferPoolItem** FoundBuffer = MemoryPool.Find(AllocSize);
		if (FoundBuffer)
		{
			AllocatedBuffer = MakeShareable(*FoundBuffer, MoveTemp(BufferDeleter));
			MemoryPool.Remove(AllocSize, *FoundBuffer);
		}
	}

	if (!AllocatedBuffer)
	{
		AllocatedBuffer = MakeShareable(new FStructuredBufferPoolItem(), MoveTemp(BufferDeleter));

		// Allocate and unlock the structured buffer on render thread.
		ENQUEUE_RENDER_COMMAND(CreatePooledBuffer)([AllocatedBuffer, AllocSize](FRHICommandListImmediate& RHICmdList)
			{
				//TRACE_CPUPROFILER_EVENT_SCOPE_STR("ExrReaderGpu.AllocBuffer_RenderThread");
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("ExrReaderGpu.AllocBuffer_RenderThread %d"), AllocSize));

				RHI_BREADCRUMB_EVENT_STAT(RHICmdList, ExrImgMediaReaderGpu_AllocateBuffer, "ExrImgGpu.MipRender.AllocateBuffer");

				{
					const FRHIBufferCreateDesc CreateDesc =
						FRHIBufferCreateDesc::CreateStructured(TEXT("ExrReaderGpu.UploadBuffer"), AllocSize, sizeof(uint16) * 2)
						.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Dynamic | EBufferUsageFlags::FastVRAM)
						.DetermineInitialState();
					AllocatedBuffer->UploadBufferRef = RHICmdList.CreateBuffer(CreateDesc);
					AllocatedBuffer->UploadBufferMapped = RHICmdList.LockBuffer(AllocatedBuffer->UploadBufferRef, 0, AllocSize, RLM_WriteOnly);

					if (bExrReaderUseUploadHeap)
					{
						// The upload buffer will remain in CopySrc state.
						RHICmdList.Transition(FRHITransitionInfo(AllocatedBuffer->UploadBufferRef, CreateDesc.InitialState, ERHIAccess::CopySrc));
					}
				}

				if (bExrReaderUseUploadHeap)
				{
					const FRHIBufferCreateDesc CreateDesc =
						FRHIBufferCreateDesc::CreateStructured(TEXT("ExrReaderGpu.DestBuffer"), AllocSize, sizeof(uint16) * 2)
						.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::FastVRAM)
						.DetermineInitialState();

					AllocatedBuffer->ShaderAccessBufferRef = RHICmdList.CreateBuffer(CreateDesc);
					AllocatedBuffer->ShaderResourceView = RHICmdList.CreateShaderResourceView(AllocatedBuffer->ShaderAccessBufferRef, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(AllocatedBuffer->ShaderAccessBufferRef));
				}
				else
				{
					AllocatedBuffer->ShaderResourceView = RHICmdList.CreateShaderResourceView(AllocatedBuffer->UploadBufferRef, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(AllocatedBuffer->UploadBufferRef));
				}

				AllocatedBuffer->AllocationReadyEvent->Trigger();
			});
	}

	// This buffer will be automatically processed and returned to StagingMemoryPool once nothing keeps reference to it.
	return AllocatedBuffer;
}

void FExrImgMediaReaderGpu::ReturnGpuBufferToPool(uint32 AllocSize, FStructuredBufferPoolItem* Buffer)
{
	FScopeLock ScopeLock(&MemoryPoolCriticalSection);
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("ExrReaderGpu.ReturnPoolItem");
	MemoryPool.Add(AllocSize, Buffer);
}


/* FExrMediaTextureSampleConverter implementation
 *****************************************************************************/

bool FExrMediaTextureSampleConverter::Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints)
{
	FScopeLock ScopeLock(&ConverterCallbacksCriticalSection);
	bool bExecutionSuccessful = false;
	if (ConvertExrBufferCallback.IsBound())
	{
		bExecutionSuccessful = ConvertExrBufferCallback.Execute(RHICmdList, InDstTexture, MipBuffers, GetParams());
	}
	return bExecutionSuccessful;
}

FStructuredBufferPoolItem::FStructuredBufferPoolItem()
{
	constexpr bool bIsManualReset = true; // Manually reset events stay triggered until reset.
	AllocationReadyEvent = FPlatformProcess::GetSynchEventFromPool(bIsManualReset);
	check(AllocationReadyEvent);
}

FStructuredBufferPoolItem::~FStructuredBufferPoolItem()
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("ExrReaderGpu.ReleasePoolItem");
	FRHICommandListImmediate::Get().UnlockBuffer(UploadBufferRef);
	UploadBufferMapped = nullptr;

	FPlatformProcess::ReturnSynchEventToPool(AllocationReadyEvent);
}

#endif //IMGMEDIA_EXR_SUPPORTED_PLATFORM && PLATFORM_WINDOWS

