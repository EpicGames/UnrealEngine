// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ExrImgMediaReader.h"

#if defined(PLATFORM_WINDOWS) && PLATFORM_WINDOWS

#include "Containers/SortedMap.h"
#include "IImgMediaReader.h"
#include "IMediaTextureSampleConverter.h"

class FExrImgMediaReaderGpu;
class FExrMediaTextureSampleConverter;
struct FNativeMediaSourceColorSettings;

struct FStructuredBufferPoolItem
{
	/**
	* This is the actual buffer reference that we need to keep after it is locked and until it is unlocked.
	* The buffer is used as an upload heap and will not be accessed by shader if CVarExrReaderUseUploadHeap is set.
	*/
	FBufferRHIRef UploadBufferRef;

	/** 
	* A pointer to mapped GPU memory.
	*/
	void* UploadBufferMapped;

	/**
	* This buffer is used by the swizzling shader if CVarExrReaderUseUploadHeap is set and UploadBufferRef contents are copied into it.
	*/
	FBufferRHIRef ShaderAccessBufferRef;

	/** 
	* Resource View used by swizzling shader.
	*/
	FShaderResourceViewRHIRef ShaderResourceView;

	/** Event used to wait for completed buffer allocations. */
	FEvent* AllocationReadyEvent;

	/* Constructor */
	FStructuredBufferPoolItem();

	/** Destructor to clean up render resources */
	~FStructuredBufferPoolItem();
};

/**
* A shared pointer that will be released automatically and returned to UsedPool
*/
typedef TSharedPtr<FStructuredBufferPoolItem, ESPMode::ThreadSafe> FStructuredBufferPoolItemSharedPtr;

/**
 * Implements a reader for EXR image sequences.
 */
class FExrImgMediaReaderGpu
	: public FExrImgMediaReader
	, public TSharedFromThis<FExrImgMediaReaderGpu, ESPMode::ThreadSafe>
{
public:

	/** Constructor. */
	FExrImgMediaReaderGpu(const TSharedRef<FImgMediaLoader, ESPMode::ThreadSafe>& InLoader, const FString& InLayerName = FString());
	virtual ~FExrImgMediaReaderGpu();

public:

	//~ FExrImgMediaReader interface
	virtual bool GetFrameInfo(const FString& ImagePath, FImgMediaFrameInfo& OutInfo) override;
	virtual bool ReadFrame(int32 FrameId, const TMap<int32, FMediaTileSelection>& InMipTiles, TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> OutFrame) override;
	
	/**
	* For performance reasons we want to pre-allocate structured buffers to at least the number of concurrent frames.
	*/
	virtual void PreAllocateMemoryPool(int32 NumFrames, const FImgMediaFrameInfo& FrameInfo) override;

protected:

	/** 
	 * This function reads file in 16 MB chunks and if it detects that
	 * Frame is pending for cancellation stops reading the file and returns false.
	*/
	EReadResult ReadInChunks(uint16* Buffer, const FString& ImagePath, int32 FrameId, const FIntPoint& Dim, int32 BufferSize);

	/**
	 * Get the size of the buffer needed to load in an image.
	 * 
	 * @param Dim Dimensions of the image.
	 * @param NumChannels Number of channels in the image.
	 */
	static SIZE_T GetBufferSize(const FIntPoint& Dim, int32 NumChannels, bool bHasTiles, const FIntPoint& TileNum);

	/**
	* Creates Sample converter to be used by Media Texture Resource.
	*/
	static void CreateSampleConverterCallback(TSharedPtr<FExrMediaTextureSampleConverter, ESPMode::ThreadSafe> SampleConverter);

	/**
	* A function that reads one mip level.
	*/
	EReadResult ReadMip
		( const int32 CurrentMipLevel
		, const FMediaTileSelection& CurrentTileSelection
		, TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> OutFrame
		, FSampleConverterParameters& ConverterParams
		, TSharedPtr<FExrMediaTextureSampleConverter, ESPMode::ThreadSafe> SampleConverter
		, const FString& ImagePath);

public:

	/** Typically we would need the (ImageResolution.x*y)*NumChannels*ChannelSize */
	virtual FStructuredBufferPoolItemSharedPtr AllocateGpuBufferFromPool(uint32 AllocSize);

	/** Either return or Add new chunk of memory to the pool based on its size. */
	void ReturnGpuBufferToPool(uint32 AllocSize, FStructuredBufferPoolItem* Buffer);

	

private:

	/** A critical section used for memory allocation and pool management. */
	FCriticalSection MemoryPoolCriticalSection;

	/** Memory pool from where we are allowed to take buffers. */
	TMultiMap<uint32, FStructuredBufferPoolItem*> MemoryPool;

	/** Frame that was last ticked so we don't tick more than once. */
	uint64 LastTickedFrameCounter;

	/** A flag indicating this reader is being destroyed, therefore memory should not be returned. */
	bool bIsShuttingDown;

	/** If true, then just use the CPU to read the file. Written once with memory_order_release, read lock-free with memory_order_acquire. */
	std::atomic_bool bFallBackToCPU;
};


/*
* These are all the required parameters to convert Buffer to Texture converter.
*/
struct FSampleConverterParameters
{
	/** General file information including its header. */
	FImgMediaFrameInfo FrameInfo;

	/** Frame Id. */
	int32 FrameId;

	/** Resolution of the highest quality mip. */
	FIntPoint FullResolution;

	/** Dimension of the tile including the overscan borders. */
	FIntPoint TileDimWithBorders;

	/** Used for rendering tiles in bulk regions per mip level. */
	TSortedMap<int32, TArray<FIntRect>> Viewports;

#if defined(PLATFORM_WINDOWS) && PLATFORM_WINDOWS
	/** Contain information about individual tiles. Used to convert buffer data into a 2D Texture.
	* The size of this array is the exact number of complete and partial tiles for each mip level.
	*/
	TArray<TArray<FExrReader::FTileDesc>> TileInfoPerMipLevel;
#endif

	/** Number of mip levels read. */
	int32 NumMipLevels;

	/** Pixel stride in bytes. I.e. 2 bytes per pixel x 3 channels = 6. */
	int32 PixelSize;

	/** Indicates if mips stored in individual files.*/
	bool bMipsInSeparateFiles;

	/** A lower quality mip will be upscaled if value is 0 or above. At 0 highest quality mip will always be read fully. */
	int32 UpscaleMip = -1;

	/** Manual source color space & encoding overrides. */
	TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> SourceColorSettings;
};


using FExrConvertBufferCallback = TDelegate<bool(FRHICommandListImmediate& RHICmdList, FTextureRHIRef RenderTargetTextureRHI, TMap<int32, FStructuredBufferPoolItemSharedPtr>& MipBuffers, const FSampleConverterParameters ConverterParams)>;

class FExrMediaTextureSampleConverter: public IMediaTextureSampleConverter
{

public:
	virtual bool Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints) override;
	virtual ~FExrMediaTextureSampleConverter() = default;
	
	void AddCallback(FExrConvertBufferCallback&& Callback) 
	{
		FScopeLock ScopeLock(&ConverterCallbacksCriticalSection);
		ConvertExrBufferCallback = Callback;

		// Copy mip buffers to be used for rendering. 
		FScopeLock Lock(&MipBufferCriticalSection);
		MipBuffersRenderThread = MipBuffers;
	};

	bool HasMipLevelBuffer(int32 RequestedMipLevel) const
	{
		FScopeLock Lock(&MipBufferCriticalSection);
		return MipBuffers.Contains(RequestedMipLevel);
	}

	FStructuredBufferPoolItemSharedPtr GetOrCreateMipLevelBuffer(int32 RequestedMipLevel, TFunction<FStructuredBufferPoolItemSharedPtr()> AllocatorFunc)
	{
		FStructuredBufferPoolItemSharedPtr Result;

		{
			FScopeLock Lock(&MipBufferCriticalSection);

			if (FStructuredBufferPoolItemSharedPtr* BufferDataPtr = MipBuffers.Find(RequestedMipLevel))
			{
				Result = *BufferDataPtr;
			}
			else
			{
				Result = MipBuffers.Add(RequestedMipLevel, AllocatorFunc());
			}
		}
		
		// Wait for render thread buffer allocations before using resources
		Result->AllocationReadyEvent->Wait();

		return Result;
	}
	
	FSampleConverterParameters GetParams()
	{
		FScopeLock ScopeLock(&ParamsCriticalSection);
		return ConverterParams;
	}

	void SetParams(const FSampleConverterParameters& InParams)
	{
		FScopeLock ScopeLock(&ParamsCriticalSection);
		ConverterParams = InParams;
	}

private:
	/** These are all required parameters to convert the buffer into texture successfully. */
	mutable FCriticalSection ParamsCriticalSection;
	FSampleConverterParameters ConverterParams;

	FCriticalSection ConverterCallbacksCriticalSection;
	FExrConvertBufferCallback ConvertExrBufferCallback;

	/** Lock to be used exclusively on reader threads.*/
	mutable FCriticalSection MipBufferCriticalSection;

	/** An array of structured buffers that are big enough to fully contain corresponding mip levels used by reader threads and transferred into MipBuffersRenderThread. */
	TMap<int32,FStructuredBufferPoolItemSharedPtr> MipBuffers;

	/** An array of structured buffers that are big enough to fully contain corresponding mip levels. Used exclusively on the Render thread. */
	TMap<int32, FStructuredBufferPoolItemSharedPtr> MipBuffersRenderThread;
};

#endif //defined(PLATFORM_WINDOWS) && PLATFORM_WINDOWS

