// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericImgMediaReader.h"
#include "ImgMediaPrivate.h"

#include "ImageUtils.h"
#include "Loader/ImgMediaLoader.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"


bool FGenericImgMediaReader::IsSupportedFormat(const ERawImageFormat::Type InFormat) const
{
	switch (InFormat)
	{
	case ERawImageFormat::BGRA8:
	case ERawImageFormat::RGBA16:
	case ERawImageFormat::RGBA16F:
		return true;
	default:
		// Other formats do not currently have a EMediaTextureSampleFormat match.
		return false;
	}
}

bool FGenericImgMediaReader::GetFrameInfoFromImage(const FString& ImagePath, const FImageInfo& ImageInfo, FImgMediaFrameInfo& OutInfo)
{
	if (!IsSupportedFormat(ImageInfo.Format))
	{
		return false;
	}

	const FString Extension = FPaths::GetExtension(ImagePath).ToLower();

	if (Extension == TEXT("bmp"))
	{
		OutInfo.FormatName = TEXT("BMP");
	}
	else if ((Extension == TEXT("jpg")) || (Extension == TEXT("jpeg")))
	{
		OutInfo.FormatName = TEXT("JPEG");
	}
	else if (Extension == TEXT("png"))
	{
		OutInfo.FormatName = TEXT("PNG");
	}
	else
	{
		UE_LOGF(LogImgMedia, Warning, "FGenericImgMediaReader: Unsupported file format in %ls", *ImagePath);
		return false;
	}

	if (ImageInfo.NumSlices > 1)
	{
		UE_LOGF(LogImgMedia, Warning, "FGenericImgMediaReader: Unsupported texture arrays or 3D textures in %ls", *ImagePath);
		return false;
	}

	// get file info
	const UImgMediaSettings* Settings = GetDefault<UImgMediaSettings>();
	OutInfo.CompressionName = TEXT("");
	OutInfo.Dim.X = ImageInfo.GetWidth();
	OutInfo.Dim.Y = ImageInfo.GetHeight();
	OutInfo.UncompressedSize = ImageInfo.GetImageSizeBytes();
	OutInfo.NumMipLevels = 1;
	OutInfo.FrameRate = Settings->DefaultFrameRate;
	OutInfo.Srgb = (ImageInfo.GetGammaSpace() == EGammaSpace::sRGB);
	OutInfo.NumChannels = ERawImageFormat::NumChannels(ImageInfo.Format);
	OutInfo.NumBytesPerPixel = static_cast<SIZE_T>(ImageInfo.GetBytesPerPixel());
	OutInfo.bHasTiles = false;
	OutInfo.TileDimensions = OutInfo.Dim;
	OutInfo.NumTiles = FIntPoint(1, 1);
	OutInfo.TileBorder = 0;

	return true;
}

/* FGenericImgMediaReader structors
 *****************************************************************************/

FGenericImgMediaReader::FGenericImgMediaReader(const TSharedRef<FImgMediaLoader, ESPMode::ThreadSafe>& InLoader)
	: LoaderPtr(InLoader)
{ }


/* FGenericImgMediaReader interface
 *****************************************************************************/

bool FGenericImgMediaReader::GetFrameInfo(const FString& ImagePath, FImgMediaFrameInfo& OutInfo)
{
	FImage Image;
	if(FImageUtils::LoadImage(*ImagePath, Image))
	{
		return GetFrameInfoFromImage(ImagePath, Image, OutInfo);
	}

	return false;
}


bool FGenericImgMediaReader::ReadFrame(int32 FrameId, const TMap<int32, FMediaTileSelection>& InMipTiles, TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> OutFrame)
{
	const TSharedPtr<FImgMediaLoader, ESPMode::ThreadSafe> Loader = LoaderPtr.Pin();
	if (Loader.IsValid() == false)
	{
		return false;
	}

	if (InMipTiles.IsEmpty())
	{
		return false;
	}

	SIZE_T BufferDataOffset = 0;
	const SIZE_T NumBytesPerPixel = Loader->GetNumBytesPerPixel();
	const int32 NumMipLevels = Loader->GetNumMipLevels();
	const FIntPoint BaseLevelDim = Loader->GetSequenceDim();
	FIntPoint CurrenDim = BaseLevelDim;

	// Loop over all mips.
	for (int32 CurrentMipLevel = 0; CurrentMipLevel < NumMipLevels; ++CurrentMipLevel)
	{
		if (InMipTiles.Contains(CurrentMipLevel))
		{
			const FMediaTileSelection& CurrentSelection = InMipTiles[CurrentMipLevel];

			// Do we want to read in this mip?
			bool ReadThisMip = !OutFrame->MipTilesPresent.Contains(CurrentMipLevel);
			if (ReadThisMip)
			{
				// Load image.
				const FString& ImagePath = Loader->GetImagePath(FrameId, CurrentMipLevel);

				FImage Image;
				const bool bSuccess = FImageUtils::LoadImage(*ImagePath, Image);

				if (!bSuccess)
				{
					UE_LOGF(LogImgMedia, Warning, "FGenericImgMediaReader: Failed to load image %ls", *ImagePath);
					return false;
				}

				FImgMediaFrameInfo Info;
				if (!GetFrameInfoFromImage(ImagePath, Image, Info))
				{
					UE_LOGF(LogImgMedia, Warning, "FGenericImgMediaReader: Failed to obtain frame info, cancelling %ls read.", *ImagePath);
					return false;
				}

				// Create the full buffer for data.
				if (!OutFrame->Data.IsValid())
				{
					SIZE_T AllocSize = Image.GetSliceSizeBytes();
					
					if (NumMipLevels > 1)
					{
						ensureMsgf(Info.Dim.X <= BaseLevelDim.X && Info.Dim.Y <= BaseLevelDim.Y, TEXT("Invalid mip dimensions, larger than sequence dimensions."));

						// We need to rely on the loader dimensions to support mips in separate folders.
						Info.Dim = BaseLevelDim;
						Info.NumMipLevels = NumMipLevels;

						const SIZE_T SizeMip0 = (SIZE_T)BaseLevelDim.X * BaseLevelDim.Y * Image.GetBytesPerPixel();
						Info.UncompressedSize = SizeMip0;
						for (int32 Level = 1; Level < Info.NumMipLevels; Level++)
						{
							Info.UncompressedSize += SizeMip0 >> (2 * Level);
						}

						// Need more space for mips.
						AllocSize = (SizeMip0 * 4) / 3;
					}

					void* Buffer = FMemory::Malloc(AllocSize, PLATFORM_CACHE_LINE_SIZE);
					OutFrame->SetInfo(Info);
					OutFrame->Data = MakeShareable(Buffer, [](void* ObjectToDelete) { FMemory::Free(ObjectToDelete); });
					OutFrame->MipTilesPresent.Reset();
					OutFrame->Stride = BaseLevelDim.X * NumBytesPerPixel;
					switch (Image.Format)
					{
					case ERawImageFormat::BGRA8: OutFrame->Format = EMediaTextureSampleFormat::CharBGRA; break;
					case ERawImageFormat::RGBA16: OutFrame->Format = EMediaTextureSampleFormat::RGBA16; break;
					case ERawImageFormat::RGBA16F: OutFrame->Format = EMediaTextureSampleFormat::FloatRGBA; break;
					default:
						UE_LOGF(LogImgMedia, Warning, "FGenericImgMediaReader: Unsupported image format, cancelling %ls read.", *ImagePath);
						return false;
					}
				}

				if (Image.GetBytesPerPixel() != NumBytesPerPixel)
				{
					UE_LOGF(LogImgMedia, Warning, "FGenericImgMediaReader: Mismatched bytes per pixel, cancelling %ls read.", *ImagePath);
					return false;
				}

				if ((Image.GetWidth() != CurrenDim.X) || (Image.GetHeight() != CurrenDim.Y))
				{
					UE_LOGF(LogImgMedia, Warning, "FGenericImgMediaReader: Mismatched expected image size, cancelling %ls read.", *ImagePath);
					return false;
				}

				// Copy data to our buffer with the right mip level offset
				FMemory::Memcpy((void*)((uint8*)OutFrame->Data.Get() + BufferDataOffset), Image.GetPixelPointer(0, 0), Image.GetSliceSizeBytes());
				OutFrame->MipTilesPresent.Emplace(CurrentMipLevel, CurrentSelection);
				OutFrame->NumTilesRead++;
			}
		}

		// Next level.
		BufferDataOffset += (CurrenDim.X * CurrenDim.Y * NumBytesPerPixel);
		CurrenDim /= 2;
	}

	return true;
}
