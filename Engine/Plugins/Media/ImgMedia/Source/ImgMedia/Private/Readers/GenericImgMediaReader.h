// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "IImgMediaReader.h"

class FImgMediaLoader;
class FRgbaInputFile;
struct FImageInfo;
struct FImgMediaFrameInfo;

namespace ERawImageFormat
{
	enum Type : uint8;
}

/**
 * Implements a reader for various image sequence formats.
 */
class FGenericImgMediaReader
	: public IImgMediaReader
{
public:

	/**
	 * Create and initialize a new instance.
	 */
	FGenericImgMediaReader(const TSharedRef<FImgMediaLoader, ESPMode::ThreadSafe>& InLoader);

	/** Virtual destructor. */
	virtual ~FGenericImgMediaReader() = default;

public:

	//~ IImgMediaReader interface

	virtual bool GetFrameInfo(const FString& ImagePath, FImgMediaFrameInfo& OutInfo) override;
	virtual bool ReadFrame(int32 FrameId, const TMap<int32, FMediaTileSelection>& InMipTiles, TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> OutFrame) override;
	virtual void CancelFrame(int32 FrameNumber) override {};
	virtual void UncancelFrame(int32 FrameNumber) override {};

private:
	/** Checks if the format is supported. */
	bool IsSupportedFormat(const ERawImageFormat::Type InFormat) const;

	/** Extract metadata from the loaded image. */
	bool GetFrameInfoFromImage(const FString& ImagePath, const FImageInfo& Image, FImgMediaFrameInfo& OutInfo);

	/** Our parent loader. */
	TWeakPtr<FImgMediaLoader, ESPMode::ThreadSafe> LoaderPtr;
};
