// Copyright Epic Games, Inc. All Rights Reserved.

#include "Readers/IImgMediaReader.h"

#include "Player/ImgMediaTextureSample.h"

TSharedPtr<FImgMediaTextureSample, ESPMode::ThreadSafe> IImgMediaReader::CreateSample()
{
	return MakeShared<FImgMediaTextureSample, ESPMode::ThreadSafe>();
}
