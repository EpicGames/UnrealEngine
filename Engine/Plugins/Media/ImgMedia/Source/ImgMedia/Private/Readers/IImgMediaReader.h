// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Assets/MediaTileSelection.h"
#include "CoreTypes.h"
#include "IMediaTextureSample.h"
#include "Math/IntPoint.h"
#include "Misc/FrameRate.h"
#include "Misc/Variant.h"
#include "RHI.h"
#include "RHIResources.h"
#include "Templates/SharedPointer.h"

class FImgMediaTextureSample;

/**
 * Information about an image sequence frame.
 */
struct FImgMediaFrameInfo
{
	/** Name of the image compression algorithm (i.e. "ZIP"). */
	FString CompressionName;

	/** Width and height of the frame (in pixels). */
	FIntPoint Dim;

	/** Name of the image format (i.e. "EXR"). */
	FString FormatName;

	/** Frame rate. */
	/*
		Note that the frame rate here is not necessarily correct if the sequence uses a frame rate override.
		The override value is not always accessible from where this member gets set, and it might also not be
		all that useful since it is conceivable that the images end up in the image cache and are either
		referenced by different media sources with different frame rate overrides, or the frame rate override
		is changed after the images have been added to the cache.
	*/
	FFrameRate FrameRate;

	/** Whether the frame is in sRGB color space. */
	bool Srgb = false;

	/** Uncompressed size (in bytes). All mip levels included. */
	SIZE_T UncompressedSize = 0;

	/** Number of channels (RGB - 3 or RGBA - 4). */
	SIZE_T NumChannels = 0;

	/** Number of bytes per pixel. */
	SIZE_T NumBytesPerPixel = 0;

	/** Does this frame have tiles?. */
	bool bHasTiles = false;

	/** Number of tiles in X and Y direction. */
	FIntPoint NumTiles;

	/** Tile dimensions. */
	FIntPoint TileDimensions;

	/** Tile border size in texels. This is required for more elaborate tile texel sampling. */
	int32 TileBorder = 0;

	/** Number of Mip Levels. */
	int32 NumMipLevels = 0;
};


/**
 * A thread-safe container for tile selections per mip level.
 */
struct FImgMediaMipTiles final
{
	/** Default constructor. */
	FImgMediaMipTiles() = default;
	/** Default destructor. */
	~FImgMediaMipTiles() = default;

	/** Move constructor. */
	FImgMediaMipTiles(FImgMediaMipTiles&& Other)
	{
		InternalMove(MoveTemp(Other));
	}

	/** Copy constructor. */
	FImgMediaMipTiles(const FImgMediaMipTiles& Other)
	{
		InternalCopy(Other);
	}

	/** Move assignment operator. */
	FImgMediaMipTiles& operator=(FImgMediaMipTiles&& Other)
	{
		InternalMove(MoveTemp(Other));
		return *this;
	}

	/** Copy assignment operator. */
	FImgMediaMipTiles& operator=(const FImgMediaMipTiles& Other)
	{
		InternalCopy(Other);
		return *this;
	}

	/**
	 * Contains the specified mip level.
	 *
	 * @param InMipLevel mip level
	 * @return true when mip level is present.
	*/
	bool Contains(const int32 InMipLevel) const
	{
		FScopeLock Lock(&CriticalSection);
		return MipTiles.Contains(InMipLevel);
	}

	/**
	 * Check if the container is empty.
	 *
	 * @return true when no mips/tiles are present.
	*/
	bool IsEmpty() const
	{
		FScopeLock Lock(&CriticalSection);
		return MipTiles.IsEmpty();
	}

	/** Reset the container. */
	void Reset()
	{
		FScopeLock Lock(&CriticalSection);
		return MipTiles.Reset();
	}

	/**
	 * Check if the existing tiles contain all of the requested ones. (Is the container a superset of requested?)
	 *
	 * @param InRequestedMipTiles Requested tile selection per mip level.
	 * @return true if the selection is entirely contained, or the request was empty.
	 */
	bool ContainsMipTiles(const TMap<int32, FMediaTileSelection>& InRequestedMipTiles)
	{
		for (auto Iter = InRequestedMipTiles.CreateConstIterator(); Iter; ++Iter)
		{
			if (!ContainsTiles(Iter.Key(), Iter.Value()))
			{
				return false;
			}
		}

		return true;
	}

	/**
	 * Check if the existing tiles contain all of the requested ones, at the specified mip level.
	 *
	 * @param InMipLevel Specified mip level.
	 * @param InSelection Requested tile selection.
	 * @return true if the selection is contained
	 */
	bool ContainsTiles(const int32 InMipLevel, const FMediaTileSelection& InSelection)
	{
		FScopeLock Lock(&CriticalSection);

		if (const FMediaTileSelection* CachedSelection = MipTiles.Find(InMipLevel))
		{
			// Check if requested tile selection is not present.
			return CachedSelection->Contains(InSelection);
		}

		// Requested mip level is not present.
		return false;
	}

	/**
	 * Include the selection at the specified mip level.
	 *
	 * @param InMipLevel Specified mip level.
	 * @param InSelection New tile selection.
	 */
	void Include(const int32 InMipLevel, const FMediaTileSelection& InSelection)
	{
		FScopeLock Lock(&CriticalSection);

		if (FMediaTileSelection* Tiles = MipTiles.Find(InMipLevel))
		{
			Tiles->Include(InSelection);
		}
		else
		{
			MipTiles.Emplace(InMipLevel, InSelection);
		}
	}

	/**
	 * Emplace the selection at the specified mip level.
	 *
	 * @param InMipLevel Specified mip level.
	 * @param InSelection New tile selection.
	 */
	void Emplace(const int32 InMipLevel, const FMediaTileSelection& InSelection)
	{
		FScopeLock Lock(&CriticalSection);

		MipTiles.Emplace(InMipLevel, InSelection);
	}

	/**
	 * Get a copy of the selection at the specified mip level.
	 *
	 * @param InMipLevel Specified mip level.
	 * @param OutSelection Returned tile selection.
	 * @return true if mip level was found
	 */
	bool GetSelection(const int32 InMipLevel, FMediaTileSelection& OutSelection) const
	{
		FScopeLock Lock(&CriticalSection);

		if (const FMediaTileSelection* Selection = MipTiles.Find(InMipLevel))
		{
			OutSelection = *Selection;
			return true;
		}

		return false;
	}

	/**
	 * Returns a calculated list of contiguous visible tile regions. Only provides regions for the missing tiles if
	 * CurrentTileSelection is specified.
	 *
	 * @param InMipLevel Specified mip level.
	 * @param InSelection Existing tile selection.
	 * @param OutRegions Visible tile regions array.
	 * @return true if mip level was found
	 */
	bool GetVisibleRegions(const int32 InMipLevel, const FMediaTileSelection& InSelection, TArray<FIntRect>& OutRegions) const
	{
		FScopeLock Lock(&CriticalSection);

		if (const FMediaTileSelection* Selection = MipTiles.Find(InMipLevel))
		{
			OutRegions = Selection->GetVisibleRegions(&InSelection);
			return true;
		}

		return false;
	}

	/** Thread-unsafe data getter: the caller is responsible for locking. */
	TMap<int32, FMediaTileSelection>& GetDataUnsafe()
	{
		return MipTiles;
	}

	/** Thread-unsafe data getter: the caller is responsible for locking. */
	const TMap<int32, FMediaTileSelection>& GetDataUnsafe() const
	{
		return MipTiles;
	}

public:

	/** Critical section to lock when using unsafe methods. */
	mutable FCriticalSection CriticalSection;

private:
	void InternalMove(FImgMediaMipTiles&& Other)
	{
		FScopeLock LockOther(&Other.CriticalSection);
		FScopeLock Lock(&CriticalSection);
		MipTiles = MoveTemp(Other.MipTiles);
	}

	void InternalCopy(const FImgMediaMipTiles& Other)
	{
		FScopeLock LockOther(&Other.CriticalSection);
		FScopeLock Lock(&CriticalSection);
		MipTiles = Other.MipTiles;
	}

	/** Tiles present per mip level. */
	TMap<int32, FMediaTileSelection> MipTiles;
};

/**
 * A single frame of an image sequence.
 */
struct FImgMediaFrame final
{
	/** Default constructor. */
	FImgMediaFrame() = default;

	/** Default destructor. */
	~FImgMediaFrame() = default;

	/** Copy constructor. */
	FImgMediaFrame(const FImgMediaFrame& Other)
		: Data(Other.Data)
		, Format(Other.Format.load())
		, MipTilesPresent(Other.MipTilesPresent)
		, NumTilesRead(Other.NumTilesRead.load())
		, Stride(Other.Stride.load())
		, Info() // copied below
		, SampleConverter() // copied below
	{
		{
			FScopeLock LockOther(&Other.SampleConverterCriticalSection);
			SampleConverter = Other.SampleConverter;
		}

		{
			FScopeLock LockOther(&Other.InfoCriticalSection);
			Info = Other.Info;
		}
	}

	/** Copy assignment operator. */
	FImgMediaFrame& operator=(const FImgMediaFrame& Other)
	{
		if (this != &Other)
		{
			this->~FImgMediaFrame();
			new(this) FImgMediaFrame(Other);
		}
		return *this;
	}

	FImgMediaFrame(FImgMediaFrame&& Other) = delete;
	FImgMediaFrame& operator=(FImgMediaFrame&& Other) = delete;

	/** The frame's data. */
	TSharedPtr<void, ESPMode::ThreadSafe> Data;

	/** The frame's sample format. */
	std::atomic<EMediaTextureSampleFormat> Format;

	/** Tiles present per mip level. */
	FImgMediaMipTiles MipTilesPresent;

	/** Total number of tiles read. */
	std::atomic<int32> NumTilesRead = 0;

	/** The frame's horizontal stride (in bytes). */
	std::atomic<uint32> Stride = 0;

	// This should only be used if you do not need to make changes to sample converter.
	IMediaTextureSampleConverter* GetSampleConverter()
	{
		return SampleConverter.Get();
	};

	template <class T>
	TSharedPtr<T, ESPMode::ThreadSafe> GetOrCreateSampleConverter()
	{
		FScopeLock ScopedLock(&SampleConverterCriticalSection);
		if (!SampleConverter.IsValid())
		{
			SampleConverter = MakeShared<T>();
		}
		return StaticCastSharedPtr<T>(SampleConverter);
	}

	/** Thread-safe info getter. */
	FImgMediaFrameInfo GetInfo() const
	{
		FScopeLock ScopedLock(&InfoCriticalSection);
		return Info;
	}

	/** Thread-safe info setter. */
	void SetInfo(FImgMediaFrameInfo InInfo)
	{
		FScopeLock ScopedLock(&InfoCriticalSection);
		Info = InInfo;
	}

	/** Thread-safe framerate getter. */
	FFrameRate GetFrameRate() const
	{
		FScopeLock ScopedLock(&InfoCriticalSection);
		return Info.FrameRate;
	}

	/** Thread-safe dimensions getter. */
	FIntPoint GetDim() const
	{
		FScopeLock ScopedLock(&InfoCriticalSection);
		return Info.Dim;
	}

	/** Thread-safe sRGB getter. */
	bool IsOutputSrgb() const
	{
		FScopeLock ScopedLock(&InfoCriticalSection);
		return Info.Srgb;
	}

	/** Thread-safe uncompressed size. */
	SIZE_T GetUncompressedSize() const
	{
		FScopeLock ScopedLock(&InfoCriticalSection);
		return Info.UncompressedSize;
	}

private:

	/** Information critical section. */
	mutable FCriticalSection InfoCriticalSection;

	/** Additional information about the frame. */
	FImgMediaFrameInfo Info;

	/** Lock to be used exclusively on reader threads. Note: this protects only the creation of the sample converter. */
	mutable FCriticalSection SampleConverterCriticalSection;

	/** Sample converter is used by Media Texture Resource to convert the texture or data. */
	TSharedPtr<IMediaTextureSampleConverter, ESPMode::ThreadSafe> SampleConverter;
};


/**
 * Interface for image sequence readers.
 */
class IImgMediaReader
{
public:

	/**
	 * Get information about an image sequence frame.
	 *
	 * @param ImagePath Path to the image file containing the frame.
	 * @param OutInfo Will contain the frame info.
	 * @return true on success, false otherwise.
	 * @see ReadFrame
	 */
	virtual bool GetFrameInfo(const FString& ImagePath, FImgMediaFrameInfo& OutInfo) = 0;

	/**
	 * Read a single image frame.
	 *
	 * @param FrameId Frame number to read.
	 * @param MipLevel Will read in this level and all higher levels.
	 * @param InTileSelection Which tiles to read.
	 * @param OutFrame Will contain the frame.
	 * @return true on success, false otherwise.
	 * @see GetFrameInfo
	 */
	virtual bool ReadFrame(int32 FrameId, const TMap<int32, FMediaTileSelection>& InMipTiles, TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> OutFrame) = 0;

	/**
	 * Mark Frame to be canceled based on Frame number. Typically this will be
	 * @param FrameNumber used for frame lookup.
	 *
	 */
	virtual void CancelFrame(int32 FrameNumber) = 0;

	/**
	 * Makes sure a frame that was passed to CancelFrame is no longer marked to be cancelled.
	 * @param FrameNumber Frame that was passed to CancelFrame.
	 */
	virtual void UncancelFrame(int32 FrameNumber) = 0;

	/**
	 * For some readers this function allows to pre-allocate enough memory to support the
	 * maximum number of frames with as much efficiency as possible.
	 *
	 */
	virtual void PreAllocateMemoryPool(int32 NumFrames, const FImgMediaFrameInfo& FrameInfo) {};

	/**
	 * Used in case reader needs to do some processing once per frame.
	 * Example: ExrImgMediaReaderGpu which returns unused memory to memory pool.
	 *
	 */
	virtual void OnTick() {};


	/** Query media metadata by name. Returns an empty variant if not available. */
	virtual FVariant GetMediaInfo(FName InfoName) const { return FVariant(); }

	/**
	 * Factory hook: produce the texture sample that the loader will hand to the
	 * media pipeline for this reader. Default returns a plain
	 * FImgMediaTextureSample. Reader subclasses can override to return a
	 * derived sample type that opts into per-reader behavior - e.g. TMV
	 * returns a sample that exposes the sparse-residency create-flags hint
	 * for its intermediate render target.
	 */
	virtual TSharedPtr<FImgMediaTextureSample, ESPMode::ThreadSafe> CreateSample();

public:

	/** Virtual destructor. */
	virtual ~IImgMediaReader() = default;
};
