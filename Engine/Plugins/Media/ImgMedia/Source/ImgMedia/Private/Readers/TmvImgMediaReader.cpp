// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvImgMediaReader.h"

#include "TmvImgMediaTextureSample.h"
#include "Async/Mutex.h"
#include "Decoder/ITmvMediaDecoder.h"
#include "Decoder/ITmvMediaDecoderFactory.h"
#include "Decoder/ITmvMediaDemuxer.h"
#include "Decoder/ITmvMediaDemuxerFactory.h"
#include "HAL/FileManager.h"
#include "ITmvMediaModule.h"
#include "ImgMediaPrivate.h"
#include "Loader/ImgMediaLoader.h"
#include "SampleConverter/TmvMediaTextureSampleConverter.h"
#include "SampleConverter/TmvMediaFrameMipBufferPool.h"
#include "SampleConverter/TmvMediaTileResidency.h"
#include "SampleConverter/TmvSharedTileSession.h"
#include "SampleConverter/TmvTileDescriptorBuilder.h"
#include "Templates/SharedPointer.h"
#include "TmvMediaTuningCVars.h"
#include "MediaPlayer.h"
#include "Utils/TmvMediaUtils.h"

namespace UE::ImgMedia
{
	static bool bTmvReaderForceOutputFullPrecision = true;	// Default to true to preserve behavior since it may affect quality.

	static FAutoConsoleVariableRef CVarbTmvReaderForceOutputFullPrecision(
		TEXT("r.TmvReader.ForceOutputFullPrecision"),
		bTmvReaderForceOutputFullPrecision,
		TEXT("Force selecting half float precision pixel format. If this is disabled, a memory conservative format with less than half float precision may be selected for YUV formats that have no alpha channel."),
		ECVF_Default);

	/**
	 * Implementation of an access unit interface that is backed by a single file.
	 */
	class FFileAccessUnit : public ITmvMediaDecoderAccessUnit
	{
	public:
		explicit FFileAccessUnit(const FString& InFilename, int32 InFrameId)
		: FrameId(InFrameId)
		, Filename(InFilename)
		, FileReader(IFileManager::Get().CreateFileReader(*InFilename))
		{
		}

		virtual ~FFileAccessUnit() override
		{
			if (FileReader)
			{
				FileReader->Close();
			}
		}

		bool IsValid() const
		{
			return FileReader.IsValid();
		}
		
		//~ Begin ITmvMediaDecoder
		virtual int64 Tell() const override
		{
			return FileReader.IsValid() ? FileReader->Tell() : 0;
		}

		virtual int64 GetTotalSize() const override
		{
			return FileReader.IsValid() ? FileReader->TotalSize() : 0;
		}

		virtual bool Seek(int64 InOffset) override
		{
			if (FileReader.IsValid())
			{
				FileReader->Seek(InOffset);
				return !FileReader->IsError();
			}
			return false;
		}

		virtual int64 Read(void *OutBuffer, int64 InSize) override
		{
			if (FileReader.IsValid())
			{
				FileReader->Serialize(OutBuffer, InSize);
				if (!FileReader->IsError())
				{
					return InSize;
				}
			}
			return 0;
		}

		virtual int32 GetFrameId() const override
		{
			return FrameId;
		}

		virtual const FString& GetFilename() const override
		{
			return Filename;
		}

		virtual FArchive* GetUnderlyingArchive() const override
		{
			return FileReader.Get();
		}
		//~ End ITmvMediaDecoder

	private:
		int32 FrameId = 0;

		/** Loaded file name */
		FString Filename;

		/** File reader */
		TUniquePtr<FArchive> FileReader;
	};

	/**
	 * Implementation of an access unit interface for a byte range within a container file.
	 * Reads directly from the container at [BaseOffset, BaseOffset + SampleSize).
	 * Tell/Seek/Read operate relative to the sample start (0-based).
	 */
	class FContainerFileAccessUnit : public ITmvMediaDecoderAccessUnit
	{
	public:
		FContainerFileAccessUnit(const FString& InContainerPath, int64 InBaseOffset, int64 InSampleSize, int32 InFrameId)
		: FrameId(InFrameId)
		, Filename(InContainerPath)
		, BaseOffset(InBaseOffset)
		, SampleSize(InSampleSize)
		, FileReader(IFileManager::Get().CreateFileReader(*InContainerPath))
		{
			if (FileReader)
			{
				FileReader->Seek(BaseOffset);
			}
		}

		virtual ~FContainerFileAccessUnit() override
		{
			if (FileReader)
			{
				FileReader->Close();
			}
		}

		bool IsValid() const
		{
			return FileReader.IsValid() && SampleSize > 0;
		}

		//~ Begin ITmvMediaDecoderAccessUnit
		virtual int64 Tell() const override
		{
			return FileReader.IsValid() ? FileReader->Tell() - BaseOffset : 0;
		}

		virtual int64 GetTotalSize() const override
		{
			return SampleSize;
		}

		virtual bool Seek(int64 InOffset) override
		{
			if (FileReader.IsValid())
			{
				FileReader->Seek(BaseOffset + InOffset);
				return !FileReader->IsError();
			}
			return false;
		}

		virtual int64 Read(void* OutBuffer, int64 InSize) override
		{
			if (FileReader.IsValid())
			{
				// Clamp read to sample boundary.
				const int64 CurrentRelative = FileReader->Tell() - BaseOffset;
				const int64 Remaining = SampleSize - CurrentRelative;
				const int64 ReadSize = FMath::Min(InSize, Remaining);
				if (ReadSize <= 0)
				{
					return 0;
				}
				FileReader->Serialize(OutBuffer, ReadSize);
				if (!FileReader->IsError())
				{
					return ReadSize;
				}
			}
			return 0;
		}

		virtual int32 GetFrameId() const override
		{
			return FrameId;
		}

		virtual const FString& GetFilename() const override
		{
			return Filename;
		}

		virtual FArchive* GetUnderlyingArchive() const override
		{
			return FileReader.Get();
		}
		//~ End ITmvMediaDecoderAccessUnit

	private:
		int32 FrameId = 0;
		FString Filename;
		int64 BaseOffset = 0;
		int64 SampleSize = 0;
		TUniquePtr<FArchive> FileReader;
	};

	/**
	* Pool for worker thread context objects that can be reused from one "ReadFrame" to the next when available.
	*/
	template<class T>
	class TTmvReaderObjectPool : public TSharedFromThis<TTmvReaderObjectPool<T>>
	{
	public:

		/** Auto clean up handle, puts the context back in the pool. */
		struct FHandle 
		{
			~FHandle()
			{
				if (TSharedPtr<TTmvReaderObjectPool> Pool = PoolWeak.Pin())
				{
					Pool->Release(ReaderObject);
				}
			}

			bool IsValid() const
			{
				return ReaderObject.IsValid();
			}

			T* operator->() const
			{
				return ReaderObject.Get();
			}

			TSharedPtr<T> ReaderObject;

		private:
			TWeakPtr<TTmvReaderObjectPool> PoolWeak;
			friend TTmvReaderObjectPool;
		};

		/**
		 * Acquire an available decoder from the pool or create a new one.
		 * @remark we assume this is for the same stream, no-sub pooling needed (yet).
		 */
		FHandle Acquire(TFunction<TSharedPtr<T>()> InAllocatorFunc)
		{
			TUniqueLock<FMutex> Lock(FreeObjectsMutex);

			FHandle Handle;
			if (!FreeObjects.IsEmpty())
			{
				Handle.ReaderObject = FreeObjects.Top();
				FreeObjects.Pop();
			}

			if (!Handle.ReaderObject)
			{
				Handle.ReaderObject = InAllocatorFunc();
			}

			Handle.PoolWeak = this->AsWeak();
			return Handle;
		}

		/** Returns unused objects to the pool. */
		void Release(const TSharedPtr<T>& InObject)
		{
			TUniqueLock<FMutex> Lock(FreeObjectsMutex);
			FreeObjects.Add(InObject);
		}

	private:
		FMutex FreeObjectsMutex;
		TArray<TSharedPtr<T>> FreeObjects;
	};

	/**
	* Pool for the decoder context so it gets reused from one "ReadFrame" to the next when available.
	* The "ReadFrame" calls are themselves on worker threads, so there may be more than one concurrent call to this.
	* @remark It is not clear how to properly load balance all the worker threads. That part is still wip.
	*/
	class FTmvDecoderPool : public TTmvReaderObjectPool<ITmvMediaDecoder>
	{
	public:
		// Acquire an available decoder from the pool or create new one.
		// Note: we assume this is for the same stream, no-sub pooling needed (yet).
		FHandle AcquireDecoder(ITmvMediaDecoderFactory& InFactory)
		{
			return Acquire([&InFactory]()
				{
					return InFactory.CreateDecoder(TEXT(""), {});
				});
		}
	};

	/**
	 * Pool for the parser context, also needed by ReadFrame.
	 */
	class FTmvParserPool : public TTmvReaderObjectPool<ITmvMediaParser>
	{
	public:
		FHandle AcquireParser(ITmvMediaDecoderFactory& InFactory)
		{
			return Acquire([&InFactory]()
				{
					return InFactory.CreateParser(TEXT(""), {});
				});
		}
	};

	/** Calculate the total memory size in bytes of the given set of mipmaps. */
	SIZE_T CalculateMemorySizeInBytes(const TArray<FTmvMediaFrameMipInfo>& InMipInfos)
	{
		SIZE_T TotalSize = 0;
		for (const FTmvMediaFrameMipInfo& MipInfo : InMipInfos)
		{
			TotalSize += MipInfo.GetMemorySizeInBytes();
		}
		return TotalSize;
	}
	
	void FillFrameInfo(
		const ITmvMediaDecoderFactory& InDecoderFactory, 
		const TArray<FTmvMediaFrameMipInfo>& InMipInfos, 
		const FTmvMediaDemuxerTrackInfo* InVideoTrackInfo,
		FImgMediaFrameInfo& OutInfo)
	{
		OutInfo.CompressionName = InDecoderFactory.GetName();	// todo: codec: APV, or one of the exr compressions.
		OutInfo.Dim = FIntPoint(InMipInfos[0].Width, InMipInfos[0].Height);
		OutInfo.NumMipLevels = InMipInfos.Num();
		OutInfo.UncompressedSize = CalculateMemorySizeInBytes(InMipInfos);
		OutInfo.FormatName = "TMV";	// todo: Could be EXR or APV.
		if (InVideoTrackInfo)
		{
			if (InVideoTrackInfo->ConstantSampleDuration > 0 && InVideoTrackInfo->Timescale > 0)
			{
				OutInfo.FrameRate = FFrameRate(InVideoTrackInfo->Timescale, InVideoTrackInfo->ConstantSampleDuration);
			}
			else if (InVideoTrackInfo->NumSamples > 0 && InVideoTrackInfo->Duration > 0)
			{
				// Fallback: average frame rate from total duration and sample count.
				OutInfo.FrameRate = FFrameRate(
					static_cast<int32>(static_cast<int64>(InVideoTrackInfo->NumSamples) * InVideoTrackInfo->Timescale / InVideoTrackInfo->Duration),
					1);
			}
		}
		else
		{
			OutInfo.FrameRate = GetDefault<UImgMediaSettings>()->DefaultFrameRate;
		}
		OutInfo.Srgb = true;	// todo: Transfer function and color space info.
		OutInfo.NumChannels = InMipInfos[0].NumComponents;
		OutInfo.NumBytesPerPixel = OutInfo.NumChannels * 2; // todo: This assumes 16 bits per channel
		OutInfo.bHasTiles = InMipInfos[0].NumTiles.X * InMipInfos[0].NumTiles.Y > 1;
		OutInfo.TileDimensions = FIntPoint(InMipInfos[0].TileWidth, InMipInfos[0].TileHeight);
		OutInfo.NumTiles = InMipInfos[0].NumTiles;
		OutInfo.TileBorder = 0;
	}

	enum class ETmvSetupResult
	{
		Fail,
		Success,
		Skipped
	};

	/** Helper: expand a list of FIntRect tile regions into a flat set of (X, Y) tile coords. */
	static void ExpandTileRegionsToSet(const TArray<FIntRect>& InRegions, TSet<FIntPoint>& OutSet)
	{
		for (const FIntRect& R : InRegions)
		{
			for (int32 Y = R.Min.Y; Y < R.Max.Y; ++Y)
			{
				for (int32 X = R.Min.X; X < R.Max.X; ++X)
				{
					OutSet.Add(FIntPoint(X, Y));
				}
			}
		}
	}

	/** Setup a mip decoding request for the given mip's tile selection. */
	ETmvSetupResult TmvSetupMipRequest(
		// ImgMedia parameters
		const int32 InCurrentMipLevel,
		bool bInVisibleTilesOnly,
		const FMediaTileSelection& InCurrentTileSelection,
		const TSharedPtr<FImgMediaFrame>& OutFrame,
		// Tmv Parameters
		const TSharedPtr<FTmvMediaTextureSampleConverter>& InSampleConverter,
		FTmvMediaFrameMipBufferPool& InConverterBufferPool,
		const FTmvMediaFrameMipInfo& InTmvMipInfo,
		FTmvMediaDecoderMipRequest& OutMipRequest
		)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TmvImgMediaReader.SetupMip %d"), InCurrentMipLevel));

		OutMipRequest.MipInfo = InTmvMipInfo;
		OutMipRequest.MipBuffer = InSampleConverter->GetOrCreateMipLevelBuffer(
			InCurrentMipLevel,
			[&InConverterBufferPool, &InTmvMipInfo]() -> FTmvMediaFrameMipBufferHandle
			{
				return InConverterBufferPool.AcquireBuffer(InTmvMipInfo);
			}
		);

		// Failed to acquire a mip buffer.
		if (!OutMipRequest.MipBuffer.IsValid())
		{
			return ETmvSetupResult::Fail;
		}

		ETmvSetupResult SetupResult;

		if (bInVisibleTilesOnly)
		{
			// Virtualized path: query the buffer's residency map, assign slots
			// to newly-visible tiles (LRU-evicting non-visible tiles when the
			// budget is full), and emit a parallel (TileCoordsToDecode,
			// TileDstSlots) pair for the decoder. TileRegions is rebuilt as
			// 1x1 rects from the actual decode set so the post-decode upload
			// (CopyTileRegions) walks the same tiles the decoder wrote.
			//
			// Read the buffer's latched allocation-time decision, NOT
			// InTmvMipInfo.IsVirtualized() (live CVars): a mid-clip toggle must
			// not make this path disagree with how the buffer was sized, or a
			// non-virtualized full decode would overrun a budgeted buffer.
			if (OutMipRequest.MipBuffer->IsAllocatedVirtualized())
			{
				FTmvMediaTileResidency& Residency = *OutMipRequest.MipBuffer->GetResidency();
				FTmvSharedTileSession Session(*OutMipRequest.MipBuffer);
				const TArray<FIntRect> FullVisible = InCurrentTileSelection.GetVisibleRegions();

				// Cheap visible-tile count from rect arithmetic.
				int32 VisibleCount = 0;
				for (const FIntRect& R : FullVisible)
				{
					VisibleCount += FMath::Max(0, R.Max.X - R.Min.X) * FMath::Max(0, R.Max.Y - R.Min.Y);
				}

				Session.EnsureCapacityFor(VisibleCount);

				// Shared-pool mode: build VisibleSet so we can publish it for
				// cross-owner eviction protection. Local mode: VisibleSet only
				// needed when an eviction can actually happen (visible > capacity).
				const bool bMightEvictLocal = !Residency.IsShared() && (VisibleCount > Residency.Capacity);
				TSet<FIntPoint> VisibleSet;
				if (Residency.IsShared() || bMightEvictLocal)
				{
					VisibleSet.Reserve(VisibleCount);
					ExpandTileRegionsToSet(FullVisible, VisibleSet);
				}

				Session.PublishVisibility(VisibleSet);

				TArray<FIntPoint> CoordsToDecode;
				TArray<int32> DstSlots;
				TArray<FIntRect> DecodedRects;
				CoordsToDecode.Reserve(VisibleCount);
				DstSlots.Reserve(VisibleCount);
				DecodedRects.Reserve(VisibleCount);
				int32 NumOverflowed = 0;

				for (const FIntRect& R : FullVisible)
				{
					for (int32 Y = R.Min.Y; Y < R.Max.Y; ++Y)
					{
						for (int32 X = R.Min.X; X < R.Max.X; ++X)
						{
							const FIntPoint Tile(X, Y);

							const FTmvSharedTileSession::FSlotResolution Resolution = Session.ResolveSlot(Tile, VisibleSet);
							if (Resolution.Result == FTmvSharedTileSession::ESlotResult::AlreadyResident)
							{
								continue;
							}
							if (Resolution.Result == FTmvSharedTileSession::ESlotResult::Overflowed)
							{
								// Hit the retry cap or grow is structurally
								// impossible (4 GB RHI ceiling, etc.). Skip the
								// tile with a one-shot overflow warning.
								++NumOverflowed;
								if (NumOverflowed == 1)
								{
									UE_LOGF(LogImgMedia, Warning,
										"TmvSetupMipRequest mip=%d: AllocateSlot returned INDEX_NONE even after blocking-grow retries. Tile (%d,%d) skipped.",
										InCurrentMipLevel, Tile.X, Tile.Y);
								}
								continue;
							}

							CoordsToDecode.Add(Tile);
							DstSlots.Add(Resolution.Slot);
							DecodedRects.Add(FIntRect(Tile, Tile + FIntPoint(1, 1)));
						}
					}
				}

				// Overflow count rides on the buffer (read back via the request's
				// MipBuffer handle) so FTmvMediaDecoderMipRequest stays ABI-frozen.
				OutMipRequest.MipBuffer->SetPendingDecodeOverflowCount(NumOverflowed);

				if (CoordsToDecode.IsEmpty() && InCurrentTileSelection.IsAnyVisible())
				{
					// All visible tiles already resident in the buffer - no
					// decode needed this call. (Caller drops the request.)
					SetupResult = ETmvSetupResult::Skipped;
				}
				else
				{
					// Routing also rides on the buffer (consumed by the decoder via
					// the MipBuffer handle) instead of the public request struct.
					OutMipRequest.MipBuffer->SetPendingDecodeRouting(MoveTemp(CoordsToDecode), MoveTemp(DstSlots));
					OutMipRequest.TileRegions = MoveTemp(DecodedRects);
					SetupResult = ETmvSetupResult::Success;
				}
				return SetupResult;
			}

			// Non-virtualized path: use MipTilesPresent's delta (or full visible set
			// if no delta cache) and natural-tile-index offsets in the upload buffer.
			TArray<FIntRect> TileRegionsToRead;
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ApvReaderGpu.CalculateRegions);

				if (!OutFrame->MipTilesPresent.GetVisibleRegions(InCurrentMipLevel, InCurrentTileSelection, TileRegionsToRead))
				{
					TileRegionsToRead = InCurrentTileSelection.GetVisibleRegions();
				}
			}

			if (TileRegionsToRead.IsEmpty() && InCurrentTileSelection.IsAnyVisible())
			{
				// If all tiles were previously read and stored in cached frame, reading can be skipped for this mip.
				SetupResult = ETmvSetupResult::Skipped;
			}
			else
			{
				OutMipRequest.TileRegions = MoveTemp(TileRegionsToRead);
				SetupResult = ETmvSetupResult::Success;
			}
		}
		else
		{
			// Add just one region covering the whole mip.
			const FIntRect FullFrameTileRegion(FIntPoint(0,0), InTmvMipInfo.NumTiles);
			OutMipRequest.TileRegions.Add(FullFrameTileRegion);
			SetupResult = ETmvSetupResult::Success;
		}
		return SetupResult;
	}

	/**
	 * Update the set of "viewports" that will be rendered by the converter. The set is calculated from the coalesced set of visible tiles for the frame.
	 */
	void UpdateTmvConverterViewports(
		const FTmvMediaTextureSampleConverterParameters& InConverterParams,
		const TSharedPtr<FTmvMediaTextureSampleConverter>& InSampleConverter,
		const TSharedPtr<FImgMediaFrame>& OutFrame,
		TSortedMap<int32, TArray<FIntRect>>& OutViewports)
	{
		FScopeLock Lock(&OutFrame->MipTilesPresent.CriticalSection);
		// Todo: Investigate - All Tiles in frame history vs current job tiles.
		//
		// This will prepare coalesced viewports from all the tiles present in the output frame, not just the tiles decoded on the current job.
		// It is unclear why this is needed, given that if the tiles have been rendered in previous jobs, they should already be rendered
		// in the destination render target.
		//
		// From Experiment: if the viewports update is limited to only the set of regions decoded in the current job,
		// it leads to a noticeable lag in the render update of the mips as queued decode jobs don't update the render job fast enough for camera moves.
		//
		// Hypothesis: The OutFrame might need an additional MipTiles set to represent what has been rendered vs decoded.
		// The set of viewports to render would the delta between MipTilesDecoded vs MipTilesRendered.
		// Decode jobs update MipTileDecoded while Render jobs update MipTilesRendered.
		for (const TPair<int32, FMediaTileSelection>& TilesPerMip : OutFrame->MipTilesPresent.GetDataUnsafe())
		{
			const FMediaTileSelection& CurrentTileSelection = TilesPerMip.Value;
			const int32 CurrentMipLevel = TilesPerMip.Key;

			// Skip this mip since we don't have anything to render.
			if (!InSampleConverter->HasMipLevelBuffer(CurrentMipLevel))
			{
				continue;
			}

			const int32 MipLevelDiv = 1 << CurrentMipLevel;
			FIntPoint CurrentMipDim = (InConverterParams.FullResolution / MipLevelDiv).ComponentMax(FIntPoint(1,1));
			const FIntRect FullFrameViewport(FIntPoint::ZeroValue, CurrentMipDim);

			// These are the coalesced visible regions for the current frame cumulative from past and current decode jobs.
			const TArray<FIntRect> VisibleRegions = CurrentTileSelection.GetVisibleRegions();

			if (!VisibleRegions.IsEmpty())
			{
				TArray<FIntRect>& Viewports = OutViewports.Add(CurrentMipLevel);
				// Smell: shouldn't the viewport set be cleared to avoid repeating/overlaping viewports from previous jobs?
				if (InConverterParams.bHasTiles)
				{
					for (const FIntRect& TileRegion : VisibleRegions)
					{
						FIntRect Viewport;
						Viewport.Min = FIntPoint(InConverterParams.TileDimWithBorders.X * TileRegion.Min.X, InConverterParams.TileDimWithBorders.Y * TileRegion.Min.Y);
						Viewport.Max = FIntPoint(InConverterParams.TileDimWithBorders.X * TileRegion.Max.X, InConverterParams.TileDimWithBorders.Y * TileRegion.Max.Y);
						Viewport.Clip(FullFrameViewport);
						Viewports.Add(Viewport);
					}
				}
				else
				{
					// Add a single viewport covering the whole frame.
					Viewports.Add(FullFrameViewport);
				}
			}
		}
	}

	TConstArrayView<FTmvMediaShaderTileDesc> GetTileInfoForMipLevel(const FTmvMediaTextureSampleConverterParameters& InConverterParams, int32 InMipLevel)
	{
		if (InConverterParams.TileInfoPerMipLevel.IsValidIndex(InMipLevel))
		{
			return InConverterParams.TileInfoPerMipLevel[InMipLevel];
		}
		// Empty array view.
		return TConstArrayView<FTmvMediaShaderTileDesc>();
	}

	/**
	 * Utility function to create a reader if the specified file is a supported container format.
	 * @return Created Reader or null if not supported.
	 */
	TSharedPtr<FTmvImgMediaReader> CreateContainerReader(
		const TSharedRef<FImgMediaLoader>& InLoader, 
		const FString& InMediaFilePath, 
		const FString& InExtension, 
		ITmvMediaModule& InTmvModule)
	{
		// Check if this is a container format supported by a registered demuxer factory.
		const TSharedPtr<ITmvMediaDemuxerFactory, ESPMode::ThreadSafe> DemuxerFactory = InTmvModule.FindDemuxerFactoryForExtension(InExtension);
		if (!DemuxerFactory.IsValid())
		{
			return nullptr;
		}
		
		// Create and open the demuxer.
		const TSharedPtr<ITmvMediaDemuxer, ESPMode::ThreadSafe> Demuxer = DemuxerFactory->CreateDemuxer();
		if (!Demuxer.IsValid())
		{
			return nullptr;
		}

		if (Demuxer->OpenFile(InMediaFilePath) != ETmvMediaContainerResult::Success)
		{
			UE_LOGF(LogImgMedia, Warning, "FTmvImgMediaReader::CreateReader: Failed to open container: %ls", *Demuxer->GetLastError());
			return nullptr;
		}

		// Find the first video track.
		int32 VideoTrackIndex = INDEX_NONE;
		FTmvMediaDemuxerTrackInfo VideoTrackInfo;
		for (int32 i = 0; i < Demuxer->GetTrackCount(); ++i)
		{
			FTmvMediaDemuxerTrackInfo Info;
			if (Demuxer->GetTrackInfo(i, Info) == ETmvMediaContainerResult::Success && Info.TrackType == ETmvMediaTrackType::Video)
			{
				VideoTrackIndex = i;
				VideoTrackInfo = Info;
				break;
			}
		}

		if (VideoTrackIndex == INDEX_NONE)
		{
			UE_LOGF(LogImgMedia, Warning, "FTmvImgMediaReader::CreateReader: No video track found in container \"%ls\".", *InMediaFilePath);
			Demuxer->Close();
			return nullptr;
		}

		// Find a decoder factory for the track's codec.
		const FString CodecFormat = UE::TmvMedia::Utils::FourCCToString(VideoTrackInfo.SampleEntryFormat);

		TSharedPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe> DecoderFactory = InTmvModule.GetBestDecoderFactoryForFormat(CodecFormat, {});
		if (!DecoderFactory.IsValid())
		{
			UE_LOGF(LogImgMedia, Warning, "FTmvImgMediaReader::CreateReader: No decoder factory for codec '%ls'.", *CodecFormat);
			Demuxer->Close();
			return nullptr;
		}
		return MakeShared<FTmvContainerImgMediaReader>(InLoader, DecoderFactory.ToSharedRef(), Demuxer, VideoTrackIndex, InMediaFilePath);
	}
}

// -------------------------------------------------------------------------------------------------
// FTmvImgMediaReader (abstract base)
// -------------------------------------------------------------------------------------------------

FTmvImgMediaReader::FTmvImgMediaReader(const TSharedRef<FImgMediaLoader>& InLoader, const TSharedRef<ITmvMediaDecoderFactory>& InDecoderFactory)
	: LoaderWeak(InLoader)
	, DecoderFactory(InDecoderFactory)
{
	using namespace UE::ImgMedia;
	DecoderPool = MakeShared<FTmvDecoderPool>();
	ParserPool = MakeShared<FTmvParserPool>();
	FrameMipBufferPool = MakeShared<FTmvMediaFrameMipBufferPool>();
}

FTmvImgMediaReader::~FTmvImgMediaReader() = default;

TArray<FString> FTmvImgMediaReader::GetSupportedImageFileExtensions()
{
	TArray<FString> OutExtensions;

	// File-sequence extensions from decoder factories.
	TArray<TWeakPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe>> DecoderFactories;
	ITmvMediaModule::GetOrLoad().GetDecoderFactories(DecoderFactories);
	for (const TWeakPtr<ITmvMediaDecoderFactory>& FactoryWeak : DecoderFactories)
	{
		if (TSharedPtr<ITmvMediaDecoderFactory> Factory = FactoryWeak.Pin())
		{
			OutExtensions.Append(Factory->GetSupportedFileExtensions());
		}
	}
	return OutExtensions;
}

TArray<FString> FTmvImgMediaReader::GetSupportedContainerFileExtensions()
{
	TArray<FString> OutExtensions;

	// Container extensions from demuxer factories.
	TArray<TWeakPtr<ITmvMediaDemuxerFactory, ESPMode::ThreadSafe>> DemuxerFactories;
	ITmvMediaModule::GetOrLoad().GetDemuxerFactories(DemuxerFactories);
	for (const TWeakPtr<ITmvMediaDemuxerFactory>& FactoryWeak : DemuxerFactories)
	{
		if (TSharedPtr<ITmvMediaDemuxerFactory> Factory = FactoryWeak.Pin())
		{
			OutExtensions.Append(Factory->GetSupportedContainerFormats());
		}
	}
	return OutExtensions;
}

bool FTmvImgMediaReader::IsImageFileExtensionSupported(const FString& InExtension)
{
	// Check decoder factories (file sequence).
	TArray<TWeakPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe>> DecoderFactories;
	ITmvMediaModule::GetOrLoad().GetDecoderFactories(DecoderFactories);
	for (const TWeakPtr<ITmvMediaDecoderFactory>& FactoryWeak : DecoderFactories)
	{
		if (TSharedPtr<ITmvMediaDecoderFactory> Factory = FactoryWeak.Pin())
		{
			if (Factory->SupportsFormat(InExtension, {}))
			{
				return true;
			}
		}
	}
	return false;
}

bool FTmvImgMediaReader::IsContainerFileExtensionSupported(const FString& InExtension)
{
	// Check demuxer factories (container).
	return ITmvMediaModule::GetOrLoad().FindDemuxerFactoryForExtension(InExtension).IsValid();
}

TSharedPtr<FImgMediaTextureSample, ESPMode::ThreadSafe> FTmvImgMediaReader::CreateSample()
{
	return MakeShared<FTmvImgMediaTextureSample, ESPMode::ThreadSafe>();
}

TSharedPtr<FTmvImgMediaReader> FTmvImgMediaReader::CreateReader(const TSharedRef<FImgMediaLoader>& InLoader, const FString& InMediaFilePath)
{
	const FString Extension = FPaths::GetExtension(InMediaFilePath);

	ITmvMediaModule& TmvModule = ITmvMediaModule::GetOrLoad();

	TSharedPtr<FTmvImgMediaReader> ContainerReader = UE::ImgMedia::CreateContainerReader(InLoader, InMediaFilePath, Extension, TmvModule);
	if (ContainerReader.IsValid())
	{
		return ContainerReader;
	}

	// Fall back to file-sequence reader.
	TSharedPtr<ITmvMediaDecoderFactory, ESPMode::ThreadSafe> Factory = TmvModule.GetBestDecoderFactoryForFormat(Extension, {});
	if (Factory.IsValid())
	{
		return MakeShared<FTmvFileSequenceImgMediaReader>(InLoader, Factory.ToSharedRef());
	}
	return nullptr;
}

bool FTmvImgMediaReader::GetFrameInfo(const FString& InImagePath, FImgMediaFrameInfo& OutInfo)
{
	check(ParserPool);

	TUniquePtr<ITmvMediaDecoderAccessUnit> AccessUnit = CreateAccessUnit(0);
	if (!AccessUnit)
	{
		UE_LOGF(LogImgMedia, Verbose, "FTmvImgMediaReader: Failed to create access unit for frame info");
		return false;
	}

	using namespace UE::ImgMedia;

	TArray<FTmvMediaFrameMipInfo> MipInfos;
	{
		FTmvParserPool::FHandle ParserHandle = ParserPool->AcquireParser(*DecoderFactory);
		if (ParserHandle->ParseMipInfos(*AccessUnit, MipInfos) != ETmvMediaDecoderResult::Success)
		{
			return false;
		}
		if (MipInfos.Num() == 0)
		{
			return false;
		}
	}

	FillFrameInfo(*DecoderFactory, MipInfos, GetVideoTrackInfo(), OutInfo);
	
	return true;
}

bool FTmvImgMediaReader::ReadFrame(int32 InFrameId, const TMap<int32, FMediaTileSelection>& InMipTiles, TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> OutFrame)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TmvImgMedia.ReadFrame %d"), InFrameId));

	const double ReadFrameStartT = FPlatformTime::Seconds();
	bool bSuccess = false;
	ON_SCOPE_EXIT
	{
		const double ReadFrameMs = (FPlatformTime::Seconds() - ReadFrameStartT) * 1000.0;
		if (ReadFrameMs > 1000.0)
		{
			UE_LOGF(LogImgMedia, Warning,
				"TmvImgMediaReader: SLOW ReadFrame frame=%d took %.0f ms (success=%d).",
				InFrameId, ReadFrameMs, (int32)bSuccess);
		}
	};

	check(ParserPool);
	check(DecoderPool);
	check(FrameMipBufferPool);

	const TSharedPtr<FImgMediaLoader, ESPMode::ThreadSafe> Loader = LoaderWeak.Pin();
	if (!Loader.IsValid() || InMipTiles.IsEmpty() || !OutFrame.IsValid())
	{
		return false;
	}

	using namespace UE::ImgMedia;

	TUniquePtr<ITmvMediaDecoderAccessUnit> AccessUnit = CreateAccessUnit(InFrameId);
	if (!AccessUnit)
	{
		UE_LOGF(LogImgMedia, Warning, "FTmvImgMediaReader: Failed to create access unit for frame %d", InFrameId);
		return false;
	}

	TArray<FTmvMediaFrameMipInfo> MipInfos;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TmvImgMedia.ParseAU);
		FTmvParserPool::FHandle ParserHandle = ParserPool->AcquireParser(*DecoderFactory);
		if (ParserHandle->ParseMipInfos(*AccessUnit, MipInfos) != ETmvMediaDecoderResult::Success)
		{
			return false;
		}
		if (MipInfos.Num() == 0)
		{
			return false;
		}
	}

	FImgMediaFrameInfo FrameInfo;
	FillFrameInfo(*DecoderFactory, MipInfos, GetVideoTrackInfo(), FrameInfo);
	OutFrame->SetInfo(FrameInfo);

	TSharedPtr<FTmvMediaTextureSampleConverter> SampleConverter = OutFrame->GetOrCreateSampleConverter<FTmvMediaTextureSampleConverter>();
	FTmvMediaTextureSampleConverterParameters ConverterParams = SampleConverter->GetParams_ThreadSafe();

	ConverterParams.FullResolution = FrameInfo.Dim;
	ConverterParams.FrameId = InFrameId;
	if (ConverterParams.FullResolution.GetMin() <= 0)
	{
		return false;
	}

	ConverterParams.bHasTiles = FrameInfo.bHasTiles;
	ConverterParams.TileDimWithBorders = FrameInfo.TileDimensions + FrameInfo.TileBorder * 2;
	ConverterParams.SourceColorSettings = Loader->GetSourceColorSettings();

	// Populate per-tile byte descriptors so the structured-buffer swizzle shader can
	// look up where each tile lives inside the mip buffer (TiledPartial path) and the
	// CPU-side partial-copy logic in FTmvMediaFrameMipStructuredBuffer::
	// CalculateBufferRegionsToCopy can compute byte regions per dirty tile.
	//
	// Only fill this when the decoder is producing a Tiled layout - for scanline
	// layout the renderer takes the PlanarFull path and ignores TileInfoPerMipLevel.
	// (TmvMediaShaderTileDesc has FIntVector4 Offsets[c] = byte offset of plane c
	// within the mip buffer, FIntVector4 Strides[c] = per-row byte stride of plane c.)
	{
		const bool bAnyTiled = MipInfos.ContainsByPredicate([](const FTmvMediaFrameMipInfo& M)
		{
			return M.Layout == ETmvMediaFrameBufferLayout::Tiled;
		});
		if (bAnyTiled)
		{
			ConverterParams.TileInfoPerMipLevel.Reset();
			ConverterParams.TileInfoPerMipLevel.SetNum(MipInfos.Num());

			for (const FTmvMediaFrameMipInfo& Mi : MipInfos)
			{
				if (Mi.Layout != ETmvMediaFrameBufferLayout::Tiled)
				{
					continue;
				}
				if (!ConverterParams.TileInfoPerMipLevel.IsValidIndex(Mi.MipLevel))
				{
					continue;
				}

				// Virtualized mips are sized+zeroed here and filled from their
				// residency map post-decode; non-virtualized mips are filled now.
				FTmvTileDescriptorBuilder::BuildNaturalIndex(Mi, ConverterParams.TileInfoPerMipLevel[Mi.MipLevel]);
			}
		}
	}

	const int32 NumMipLevels = Loader->GetNumMipLevels();

	// Force mip level to be upscaled to all higher quality mips.
	TMap<int32, FMediaTileSelection> MipTilesCopy = InMipTiles;
	int32 MipToUpscale = FMath::Clamp(Loader->GetMinimumLevelToUpscale(), -1, NumMipLevels - 1);

	// Force MipToUpscale to the smallest mip when the shader's overflow-fallback
	// path (discard for unmapped tiles) can fire and we have no project-supplied
	// upscale layer. The decision (and its post-overflow hysteresis) lives in
	// FTmvOverflowHysteresis.
	const bool bForceUpscaleForFallback = OverflowHysteresis.ShouldForceUpscaleFallback();
	if (bForceUpscaleForFallback && MipToUpscale < 0 && NumMipLevels > 0)
	{
		MipToUpscale = NumMipLevels - 1;
	}

	if (NumMipLevels > 1 && MipToUpscale >= 0)
	{
		ConverterParams.UpscaleMip = MipToUpscale;

		FMediaTileSelection FullSelection = FMediaTileSelection::CreateForTargetMipLevel(ConverterParams.FullResolution, FrameInfo.TileDimensions, MipToUpscale, true);
		if (MipTilesCopy.Contains(MipToUpscale))
		{
			MipTilesCopy[MipToUpscale] = FullSelection;
		}
		else
		{
			MipTilesCopy.Add(MipToUpscale, FullSelection);
		}
	}

	// Get a decoder for this access unit.
	FTmvDecoderPool::FHandle DecoderHandle = DecoderPool->AcquireDecoder(*DecoderFactory);
	if (!DecoderHandle.IsValid())
	{
		return false;
	}

	TArray<FTmvMediaDecoderMipRequest> MipRequests;
	MipRequests.Reserve(MipTilesCopy.Num());

	// Publish empty owner-visibility on every cached mip buffer NOT used this
	// frame, so the shared pool can evict their previously-pinned tiles when
	// other plates need slots. Without this, a plate transitioning from
	// using mips 1+2 (close) to mips 4+5 (far) leaves stale VisibleTiles
	// snapshots in the mip-1/mip-2 pools; those slots stay non-evictable
	// forever, other plates can't reuse them, and the pool grows beyond
	// actual concurrent demand.
	{
		TSet<int32> ActiveMips;
		ActiveMips.Reserve(MipTilesCopy.Num());
		for (const TPair<int32, FMediaTileSelection>& Pair : MipTilesCopy)
		{
			if (Pair.Value.IsAnyVisible())
			{
				ActiveMips.Add(Pair.Key);
			}
		}
		SampleConverter->ClearInactiveOwnerVisibility(ActiveMips);
	}

	for (const TPair<int32, FMediaTileSelection>& TilesPerMip : MipTilesCopy)
	{
		const int32 CurrentMipLevel = TilesPerMip.Key;
		const FMediaTileSelection& CurrentTileSelection = TilesPerMip.Value;

		if (!CurrentTileSelection.IsAnyVisible())
		{
			continue;
		}

		// Find the mip info corresponding to this mip level.
		FTmvMediaFrameMipInfo* MipInfo = MipInfos.FindByPredicate([CurrentMipLevel](const FTmvMediaFrameMipInfo& InMipInfo)
		{
			return CurrentMipLevel == InMipInfo.MipLevel;
		});

		if (!MipInfo)
		{
			UE_LOGF(LogImgMedia, Verbose, "TmvImgMediaReader: Missing mip %d for frame %d.", CurrentMipLevel, InFrameId);
			break;
		}

		const ETmvSetupResult Result = TmvSetupMipRequest(
			CurrentMipLevel,
			FrameInfo.bHasTiles,
			CurrentTileSelection,
			OutFrame,
			SampleConverter,
			*FrameMipBufferPool,
			*MipInfo,
			MipRequests.Emplace_GetRef());

		switch (Result)
		{
		case ETmvSetupResult::Success:
			break;
		case ETmvSetupResult::Fail:
			return false;	// Abort further reading (for now)
		case ETmvSetupResult::Skipped:
			// No new tiles need to be read, continue to the next mip level.
			MipRequests.RemoveAt(MipRequests.Num()-1, EAllowShrinking::No);
			break;
		default:
			checkNoEntry();
			break;
		}
	}

	// Update the per-clip overflow hysteresis counter from this frame's total
	// overflow count across all mip requests.
	int32 TotalOverflowed = 0;
	for (const FTmvMediaDecoderMipRequest& MipRequest : MipRequests)
	{
		// Overflow count now rides on the (Internal) mip buffer rather than the
		// ABI-frozen request struct. Non-virtualized buffers never set it (stays 0).
		if (MipRequest.MipBuffer.IsValid())
		{
			TotalOverflowed += MipRequest.MipBuffer->GetPendingDecodeOverflowCount();
		}
	}
	OverflowHysteresis.RecordFrame(TotalOverflowed);

	const double DecodeStartT = FPlatformTime::Seconds();
	ETmvMediaDecoderResult DecodeResult = DecoderHandle->Decode(*AccessUnit, MipRequests);
	const double DecodeMs = (FPlatformTime::Seconds() - DecodeStartT) * 1000.0;
	if (DecodeMs > 200.0)
	{
		// Anything past ~200 ms is suspicious for what should be a sub-frame
		// operation. Logs help correlate against the 10s ImgMedia BlockingFetch
		// timeout we keep seeing during multi-plate appear events.
		UE_LOGF(LogImgMedia, Warning,
			"TmvImgMediaReader: SLOW Decode for frame %d took %.0f ms (mip requests=%d, overflows=%d).",
			InFrameId, DecodeMs, MipRequests.Num(), TotalOverflowed);
	}
	if (DecodeResult != ETmvMediaDecoderResult::Success)
	{
		UE_LOGF(LogImgMedia, Verbose, "TmvImgMediaReader: Failed tile/mip decoding request for frame %d.", InFrameId);
		return false;
	}

	// For virtualized mips, rebuild TileInfoPerMipLevel from the buffer's
	// residency map so the upload (CopyTileRegions) and the swizzle shader
	// see slot-based byte offsets instead of natural-tile-index offsets.
	// Must run AFTER decode (decoder doesn't read this) and BEFORE
	// CopyTileRegions / SetParams_ThreadSafe.
	for (const FTmvMediaDecoderMipRequest& MipRequest : MipRequests)
	{
		const FTmvMediaFrameMipInfo& Mi = MipRequest.MipInfo;
		// Use the buffer's latched decision, not Mi.IsVirtualized() (live CVars):
		// the decode that just ran could span a CVar toggle, and the rebuild must
		// match the buffer the decoder actually wrote into.
		if (!MipRequest.MipBuffer.IsValid() || !MipRequest.MipBuffer->IsAllocatedVirtualized())
		{
			continue;
		}
		if (!ConverterParams.TileInfoPerMipLevel.IsValidIndex(Mi.MipLevel))
		{
			continue;
		}

		const FTmvMediaTileResidency* Residency = MipRequest.MipBuffer->GetResidency();
		if (!Residency)
		{
			continue;
		}
		FTmvTileDescriptorBuilder::BuildFromResidency(Mi, *Residency, ConverterParams.TileInfoPerMipLevel[Mi.MipLevel]);
	}

	for (const FTmvMediaDecoderMipRequest& MipRequest : MipRequests)
	{
		int32 MipLevel = MipRequest.MipInfo.MipLevel;
		if (MipRequest.OutResult == ETmvMediaDecoderResult::Success)
		{
			OutFrame->MipTilesPresent.Include(MipLevel, MipTilesCopy[MipLevel]);
			OutFrame->NumTilesRead += MipRequest.OutNumTilesDecoded;

			// Upload newly decoded tiles to GPU. Shared-pool buffers consult their
			// own CaptureWriteState() snapshot (taken under the decoder's read lock)
			// so the per-tile copy targets the EXACT upload+SA pair the decoder
			// wrote into. Critical when a concurrent grow swapped the pool's
			// current refs between decode and this call (the green-frame race).
			MipRequest.MipBuffer->CopyTileRegions(
				InFrameId, MipRequest.MipInfo, MipRequest.TileRegions,
				GetTileInfoForMipLevel(ConverterParams, MipLevel));
		}
		else
		{
			// todo: error reporting of the failure reason (decoder error api).
			UE_LOGF(LogImgMedia, Verbose, "TmvImgMediaReader: Failed tile/mip decoding request for mip %d, frame %d.", MipLevel, InFrameId);
		}
	}

	// Create viewport(s) with all mip/tiles present in the frame.
	UpdateTmvConverterViewports(ConverterParams, SampleConverter, OutFrame, ConverterParams.Viewports);

	// Also publish the FULL visible region per mip (intended coverage from this
	// frame's tile selection, regardless of how much actually decoded). The
	// converter's coarse-current fallback compares this against the decoded
	// Viewports to fill only under-decoded mips (so good frames pay nothing).
	ConverterParams.FullVisibleViewports.Reset();
	for (const TPair<int32, FMediaTileSelection>& TilesPerMip : MipTilesCopy)
	{
		const int32 CurrentMipLevel = TilesPerMip.Key;
		if (!SampleConverter->HasMipLevelBuffer(CurrentMipLevel))
		{
			continue;
		}
		const TArray<FIntRect> VisibleRegions = TilesPerMip.Value.GetVisibleRegions();
		if (VisibleRegions.IsEmpty())
		{
			continue;
		}
		const int32 MipLevelDiv = 1 << CurrentMipLevel;
		const FIntPoint CurrentMipDim = (ConverterParams.FullResolution / MipLevelDiv).ComponentMax(FIntPoint(1, 1));
		const FIntRect FullFrameViewport(FIntPoint::ZeroValue, CurrentMipDim);
		TArray<FIntRect>& OutVp = ConverterParams.FullVisibleViewports.Add(CurrentMipLevel);
		if (ConverterParams.bHasTiles)
		{
			for (const FIntRect& TileRegion : VisibleRegions)
			{
				FIntRect Viewport;
				Viewport.Min = FIntPoint(ConverterParams.TileDimWithBorders.X * TileRegion.Min.X, ConverterParams.TileDimWithBorders.Y * TileRegion.Min.Y);
				Viewport.Max = FIntPoint(ConverterParams.TileDimWithBorders.X * TileRegion.Max.X, ConverterParams.TileDimWithBorders.Y * TileRegion.Max.Y);
				Viewport.Clip(FullFrameViewport);
				OutVp.Add(Viewport);
			}
		}
		else
		{
			OutVp.Add(FullFrameViewport);
		}
	}

	// The final output frame after the sample converter is RGBA.
	const bool bUseRGBAFormat = MipInfos[0].NumComponents >= 4 || bTmvReaderForceOutputFullPrecision; 
	OutFrame->Format =  bUseRGBAFormat ? EMediaTextureSampleFormat::FloatRGBA : EMediaTextureSampleFormat::FloatRGB;
	const EPixelFormat PixelFormat = bUseRGBAFormat ? PF_FloatRGBA : PF_FloatRGB;
	const FPixelFormatInfo& PixelFormatInfo = GPixelFormats[PixelFormat];
	OutFrame->Stride = ConverterParams.FullResolution.X * PixelFormatInfo.BlockBytes;

	SampleConverter->SetParams_ThreadSafe(ConverterParams);

	UE_LOGF(LogImgMedia, Verbose, "Reader %p: Read Pixels Complete. %i", this, InFrameId);
	bSuccess = true;
	return true;
}

// -------------------------------------------------------------------------------------------------
// FTmvFileSequenceImgMediaReader
// -------------------------------------------------------------------------------------------------

FTmvFileSequenceImgMediaReader::FTmvFileSequenceImgMediaReader(const TSharedRef<FImgMediaLoader>& InLoader, const TSharedRef<ITmvMediaDecoderFactory>& InDecoderFactory)
	: FTmvImgMediaReader(InLoader, InDecoderFactory)
{
}

TUniquePtr<ITmvMediaDecoderAccessUnit> FTmvFileSequenceImgMediaReader::CreateAccessUnit(int32 InFrameId)
{
	const TSharedPtr<FImgMediaLoader, ESPMode::ThreadSafe> Loader = LoaderWeak.Pin();
	if (!Loader.IsValid())
	{
		return nullptr;
	}

	const FString& ImagePath = Loader->GetImagePath(InFrameId, 0);
	auto AccessUnit = MakeUnique<UE::ImgMedia::FFileAccessUnit>(ImagePath, InFrameId);
	if (!AccessUnit->IsValid())
	{
		UE_LOGF(LogImgMedia, Warning, "FTmvFileSequenceImgMediaReader: Failed to open file %ls", *ImagePath);
		return nullptr;
	}
	return AccessUnit;
}

// -------------------------------------------------------------------------------------------------
// FTmvContainerImgMediaReader
// -------------------------------------------------------------------------------------------------

FTmvContainerImgMediaReader::FTmvContainerImgMediaReader(
	const TSharedRef<FImgMediaLoader>& InLoader,
	const TSharedRef<ITmvMediaDecoderFactory>& InDecoderFactory,
	const TSharedPtr<ITmvMediaDemuxer, ESPMode::ThreadSafe>& InDemuxer,
	int32 InTrackIndex,
	const FString& InContainerFilePath)
	: FTmvImgMediaReader(InLoader, InDecoderFactory)
	, Demuxer(InDemuxer)
	, DemuxerTrackIndex(InTrackIndex)
	, ContainerFilePath(InContainerFilePath)
{
	Demuxer->GetTrackInfo(DemuxerTrackIndex, VideoTrackInfo);
}

FVariant FTmvContainerImgMediaReader::GetMediaInfo(FName InfoName) const
{
	if (Demuxer.IsValid())
	{
		if (InfoName == UMediaPlayer::MediaInfoNameStartTimecodeValue.Resolve())
		{
			TOptional<FString> TC = Demuxer->GetStartTimecode();
			if (TC.IsSet())
			{
				return FVariant(TC.GetValue());
			}
		}
		else if (InfoName == UMediaPlayer::MediaInfoNameStartTimecodeFrameRate.Resolve())
		{
			TOptional<FString> Rate = Demuxer->GetStartTimecodeRate();
			if (Rate.IsSet())
			{
				return FVariant(Rate.GetValue());
			}
		}
	}
	return FVariant();
}

FTmvContainerImgMediaReader::~FTmvContainerImgMediaReader()
{
	if (Demuxer.IsValid())
	{
		Demuxer->Close();
	}
}

TUniquePtr<ITmvMediaDecoderAccessUnit> FTmvContainerImgMediaReader::CreateAccessUnit(int32 InFrameId)
{
	using namespace UE::ImgMedia;

	FTmvMediaDemuxerSample SampleInfo;
	{
		FScopeLock Lock(&DemuxerCS);
		if (Demuxer->SeekToSample(DemuxerTrackIndex, static_cast<uint32>(InFrameId)) != ETmvMediaContainerResult::Success)
		{
			UE_LOGF(LogImgMedia, Warning, "FTmvContainerImgMediaReader: Failed to seek to sample %d", InFrameId);
			return nullptr;
		}
		if (Demuxer->ReadSampleInfo(DemuxerTrackIndex, SampleInfo) != ETmvMediaContainerResult::Success)
		{
			UE_LOGF(LogImgMedia, Warning, "FTmvContainerImgMediaReader: Failed to read sample info for frame %d", InFrameId);
			return nullptr;
		}
	}

	auto AccessUnit = MakeUnique<FContainerFileAccessUnit>(ContainerFilePath, SampleInfo.FileOffset, SampleInfo.SampleSize, InFrameId);
	if (!AccessUnit->IsValid())
	{
		UE_LOGF(LogImgMedia, Warning, "FTmvContainerImgMediaReader: Failed to open container access unit for frame %d", InFrameId);
		return nullptr;
	}
	return AccessUnit;
}
