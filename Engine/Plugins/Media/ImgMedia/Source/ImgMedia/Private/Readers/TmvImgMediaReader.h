// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Decoder/ITmvMediaDemuxer.h"
#include "IImgMediaReader.h"
#include "Readers/TmvOverflowHysteresis.h"

class FImgMediaLoader;
class FTmvMediaFrameMipBufferPool;
class ITmvMediaDecoderAccessUnit;
class ITmvMediaDecoderFactory;
class ITmvMediaDemuxer;
class ITmvMediaDemuxerFactory;

namespace UE::ImgMedia
{
	class FTmvParserPool;
	class FTmvDecoderPool;
}

/**
 * Abstract base class for TMV image media readers.
 * Provides the common decode pipeline (parse, decode, compose).
 * Subclasses provide the access unit for each frame.
 */
class FTmvImgMediaReader : public IImgMediaReader
{
public:
	FTmvImgMediaReader(const TSharedRef<FImgMediaLoader>& InLoader, const TSharedRef<ITmvMediaDecoderFactory>& InDecoderFactory);
	virtual ~FTmvImgMediaReader() override;

	/** Get the list of supported image file extensions for TMV decoders. */
	static TArray<FString> GetSupportedImageFileExtensions();
	
	/** Get the list of supported container file extensions for TMV demuxers. */
	static TArray<FString> GetSupportedContainerFileExtensions();

	/**
	 * Returns true if the given image file extension is supported by a Tmv Reader. 
	 */
	static bool IsImageFileExtensionSupported(const FString& InExtension);

	/**
	 * Returns true if the given container file extension is supported by a Tmv Reader. 
	 */
	static bool IsContainerFileExtensionSupported(const FString& InExtension);

	/** 
	 * Creates a reader of the appropriate type depending on the context (input file, settings, etc).
	 * @param InLoader Parent Img Loader
	 * @param InMediaFilePath Full path of the media file, either the first image of the sequence or the container.
	 */
	static TSharedPtr<FTmvImgMediaReader> CreateReader(const TSharedRef<FImgMediaLoader>& InLoader, const FString& InMediaFilePath);
	
	/**
	 * Returns the video track info if present, nullptr otherwise.
	 */
	virtual const FTmvMediaDemuxerTrackInfo* GetVideoTrackInfo() const
	{
		return nullptr;
	}

public:
	//~ Begin IImgMediaReader
	virtual bool GetFrameInfo(const FString& InImagePath, FImgMediaFrameInfo& OutInfo) override;
	virtual bool ReadFrame(int32 InFrameId, const TMap<int32, FMediaTileSelection>& InMipTiles, TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> OutFrame) override;
	virtual void CancelFrame(int32 InFrameNumber) override {}
	virtual void UncancelFrame(int32 InFrameNumber) override {}
	virtual TSharedPtr<FImgMediaTextureSample, ESPMode::ThreadSafe> CreateSample() override;
	//~ End IImgMediaReader

protected:
	/**
	 * Create an access unit for the given frame.
	 * @param InFrameId Frame index.
	 * @return A valid access unit, or null on failure.
	 */
	virtual TUniquePtr<ITmvMediaDecoderAccessUnit> CreateAccessUnit(int32 InFrameId) = 0;

	/** Parent loader. */
	TWeakPtr<FImgMediaLoader> LoaderWeak;

	/** Decoder factory to create decoders on demand. */
	TSharedRef<ITmvMediaDecoderFactory> DecoderFactory;

	/** Parser Pool for this image reader. We need a parser per worker thread. */
	TSharedPtr<UE::ImgMedia::FTmvParserPool> ParserPool;

	/** Decoder Pool for this image reader. We need a decoder per worker thread. */
	TSharedPtr<UE::ImgMedia::FTmvDecoderPool> DecoderPool;

	/** Texture Sample Buffer Pool for this image reader. */
	TSharedPtr<FTmvMediaFrameMipBufferPool> FrameMipBufferPool;

	/** Residency-overflow upscale-fallback hysteresis (see FTmvOverflowHysteresis). */
	FTmvOverflowHysteresis OverflowHysteresis;
};

/**
 * TMV reader for file sequences: one file per frame.
 */
class FTmvFileSequenceImgMediaReader final : public FTmvImgMediaReader
{
public:
	FTmvFileSequenceImgMediaReader(const TSharedRef<FImgMediaLoader>& InLoader, const TSharedRef<ITmvMediaDecoderFactory>& InDecoderFactory);

protected:
	//~ Begin FTmvImgMediaReader
	virtual TUniquePtr<ITmvMediaDecoderAccessUnit> CreateAccessUnit(int32 InFrameId) override;
	//~ End FTmvImgMediaReader
};

/**
 * TMV reader for container files: reads frames from a .tmv container via ITmvMediaDemuxer.
 * Uses ReadSampleInfo (metadata only) + direct file reads (no intermediate memory buffer).
 */
class FTmvContainerImgMediaReader final : public FTmvImgMediaReader
{
public:
	FTmvContainerImgMediaReader(
		const TSharedRef<FImgMediaLoader>& InLoader,
		const TSharedRef<ITmvMediaDecoderFactory>& InDecoderFactory,
		const TSharedPtr<ITmvMediaDemuxer, ESPMode::ThreadSafe>& InDemuxer,
		int32 InTrackIndex,
		const FString& InContainerFilePath);

	virtual ~FTmvContainerImgMediaReader() override;

	//~ Begin FTmvImgMediaReader
	virtual const FTmvMediaDemuxerTrackInfo* GetVideoTrackInfo() const override
	{
		return (VideoTrackInfo.TrackType == ETmvMediaTrackType::Video) ? &VideoTrackInfo : nullptr;
	}
	//~ End FTmvImgMediaReader

	//~ Begin IImgMediaReader
	virtual FVariant GetMediaInfo(FName InfoName) const override;
	//~ End IImgMediaReader

protected:
	//~ Begin FTmvImgMediaReader
	virtual TUniquePtr<ITmvMediaDecoderAccessUnit> CreateAccessUnit(int32 InFrameId) override;
	//~ End FTmvImgMediaReader

private:
	/** Demuxer for reading sample info from the container. */
	TSharedPtr<ITmvMediaDemuxer, ESPMode::ThreadSafe> Demuxer;

	/** Serializes SeekToSample + ReadSampleInfo on the demuxer. */
	FCriticalSection DemuxerCS;

	/** Video track index in the demuxer. */
	int32 DemuxerTrackIndex = 0;

	/** Cached video track info from the demuxer. */
	FTmvMediaDemuxerTrackInfo VideoTrackInfo;

	/** Path to the container file (for creating per-thread file access units). */
	FString ContainerFilePath;
};
