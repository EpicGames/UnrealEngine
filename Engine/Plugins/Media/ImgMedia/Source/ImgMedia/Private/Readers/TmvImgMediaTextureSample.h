// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Player/ImgMediaTextureSample.h"
#include "RHI.h"
#include "RHIDefinitions.h"
#include "TmvMediaTuningCVars.h"

/**
 * TMV-specific image media texture sample.
 *
 * Identical to FImgMediaTextureSample for all data-plane behavior. The only
 * difference is that this sample opts the intermediate render target into a
 * D3D12 reserved-resource allocation when sparse output residency is enabled
 * and the texture is large enough to benefit. The actual per-mip commit /
 * decommit is driven by FTmvSparseOutputManager from the converter; this
 * sample just signals "create the texture in a way that the manager can
 * manage" via the engine's CreateIntermediateRenderTarget hint hook.
 *
 * Eligibility is decided here (per sample) rather than globally so that:
 *  - HD-sized plates skip the sparse path entirely (no virtual allocation,
 *    no tracking overhead) when their largest mip is below the byte
 *    threshold.
 *  - The decision is bit-identical across the lifetime of the sample (it's
 *    a function of dimensions + format, not runtime state).
 */
class FTmvImgMediaTextureSample : public FImgMediaTextureSample
{
public:
	FTmvImgMediaTextureSample() = default;

	//~ Begin IMediaTextureSample interface
	virtual ETextureCreateFlags GetIntermediateTargetCreateFlagsHint() const override
	{
		if (!UE::TmvMedia::IsSparseOutputMipEnabled())
		{
			return ETextureCreateFlags::None;
		}
		if (!GRHIGlobals.ReservedResources.Supported)
		{
			return ETextureCreateFlags::None;
		}

		// Threshold check: is the largest mip big enough to bother with
		// reserved-resource bookkeeping? Derive bytes/pixel from this sample's
		// actual output format so the estimate matches the pixel format the
		// converter's intermediate target is created with: FloatRGBA = 8 B/px,
		// or the opt-in memory-saving FloatRGB (r.TmvReader.ForceOutputFullPrecision 0)
		// = 4 B/px. Keeps this test consistent with FTmvSparseOutputManager,
		// which keys eligibility off GPixelFormats[Desc.Format].BlockBytes.
		const FIntPoint Dim = GetOutputDim();
		if (Dim.X <= 0 || Dim.Y <= 0)
		{
			return ETextureCreateFlags::None;
		}

		const EPixelFormat PixelFormat = (GetFormat() == EMediaTextureSampleFormat::FloatRGB) ? PF_FloatRGB : PF_FloatRGBA;
		const int64 BytesPerPixel = GPixelFormats[PixelFormat].BlockBytes;
		const int64 Mip0Bytes = static_cast<int64>(Dim.X) * static_cast<int64>(Dim.Y) * BytesPerPixel;
		const int64 ThresholdBytes = static_cast<int64>(UE::TmvMedia::GetSparseOutputMipMinBytesMB()) * 1024ll * 1024ll;
		if (Mip0Bytes < ThresholdBytes)
		{
			return ETextureCreateFlags::None;
		}

		return ETextureCreateFlags::ReservedResource;
	}
	//~ End IMediaTextureSample interface
};
