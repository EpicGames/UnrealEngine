// Copyright Epic Games, Inc. All Rights Reserved.

#include "Readers/TmvOverflowHysteresis.h"

#include "TmvMediaFrameInfo.h"
#include "TmvMediaTuningCVars.h"

void FTmvOverflowHysteresis::RecordFrame(int32 InTotalOverflowed)
{
	if (InTotalOverflowed > 0)
	{
		ConsecutiveCleanFrames.store(0, std::memory_order_relaxed);
		return;
	}

	const int32 Current = ConsecutiveCleanFrames.load(std::memory_order_relaxed);
	// Saturate to avoid overflow on long-running clips.
	if (Current < CleanFrameSaturation)
	{
		ConsecutiveCleanFrames.store(Current + 1, std::memory_order_relaxed);
	}
}

bool FTmvOverflowHysteresis::ShouldForceUpscaleFallback() const
{
	if (!UE::TmvMedia::IsVirtualizationEnabled())
	{
		return false;
	}

	// Grow-on-demand off: deterministic budget, overflows are persistent across
	// frames, so the fallback must stay on whenever virtualization is enabled.
	if (!UE::TmvMedia::IsGrowOnDemandEnabled())
	{
		return true;
	}

	// Shared-pool mode, transient overflow: when many plates suddenly become
	// visible, every per-mip pool can hit AllocateSlot==INDEX_NONE in the same
	// frame (slots pinned to one owner's visible set, nothing evictable until
	// the next grow lands). Keep the fallback on for a short window after the
	// last overflow so the pool can grow and residency settle.
	return UE::TmvMedia::IsSharedTilePoolEnabled()
		&& ConsecutiveCleanFrames.load(std::memory_order_relaxed) < FallbackHysteresisFrames;
}
