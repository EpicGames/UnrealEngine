// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/NumericLimits.h"

#include <atomic>

/**
 * Owns the residency-overflow upscale-fallback hysteresis that used to live as
 * a bare atomic on FTmvImgMediaReader.
 *
 * When mip virtualization / the shared tile pool can't fit every visible tile
 * in a frame, the swizzle shader discards the unmapped tiles. To avoid showing
 * a hole, the reader forces the coarsest mip as a full-screen upscale source.
 * That extra draw is expensive (~10 ms/frame across 9 plates), so it is gated
 * to a short window after the last overflow rather than left always-on.
 *
 * Thread-safety: RecordFrame runs from ReadFrame on multiple worker threads
 * concurrently (different access units, same reader). The counter is atomic;
 * the increment/reset ordering races are tolerable because the fallback
 * decision only needs approximately-fresh state - a one-frame drift either
 * direction is invisible.
 */
class FTmvOverflowHysteresis
{
public:
	/**
	 * Fold this frame's total overflow count into the clean-frame counter:
	 * reset to zero on any overflow, saturating-increment otherwise.
	 */
	void RecordFrame(int32 InTotalOverflowed);

	/**
	 * True when the coarsest-mip upscale fallback should be forced this frame:
	 * virtualization is on and either grow-on-demand is off (overflows are
	 * persistent) or we are inside the post-overflow hysteresis window in
	 * shared-pool mode. Reads the virtualization CVars itself.
	 */
	bool ShouldForceUpscaleFallback() const;

private:
	/** Frames the fallback stays on after the last overflow. */
	static constexpr int32 FallbackHysteresisFrames = 10;

	/** Saturation cap; anything past the threshold means the same thing ("fallback off"). */
	static constexpr int32 CleanFrameSaturation = 1024;

	/** Consecutive frames with no AllocateSlot==INDEX_NONE in any mip request. */
	std::atomic<int32> ConsecutiveCleanFrames{TNumericLimits<int32>::Max()};
};
