// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FImgMediaSchedulerThread;
class IQueuedWork;


class IImgMediaScheduler
{
public:

	/**
	 * Get the next available work item from the scheduler.
	 *
	 * If no work is available, the calling thread will be returned to the thread pool.
	 *
	 * @param Thread The thread that is requesting the work.
	 * @return Work item, or nullptr if no work available.
	 */
	virtual IQueuedWork* GetWorkOrReturnToPool(FImgMediaSchedulerThread* Thread) = 0;

public:

	/** Virtual destructor. */
	virtual ~IImgMediaScheduler() = default;
};
