// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImgMediaScheduler.h"

#include "GenericPlatform/GenericPlatformAffinity.h"
#include "HAL/PlatformProcess.h"
#include "Misc/ScopeLock.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"

#include "Loader/ImgMediaLoader.h"
#include "Scheduler/ImgMediaSchedulerThread.h"
#include "ImgMediaSettings.h"


/** Minimum stack size for worker threads. */
#define IMGMEDIA_SCHEDULERTHREAD_MIN_STACK_SIZE 128 * 1024


/* FImgMediaLoaderScheduler structors
 *****************************************************************************/

FImgMediaScheduler::FImgMediaScheduler()
	: LoaderRoundRobin(INDEX_NONE)
{ }


FImgMediaScheduler::~FImgMediaScheduler()
{
	Shutdown();
}


/* FImgMediaLoaderScheduler interface
 *****************************************************************************/

void FImgMediaScheduler::Initialize()
{
	if (AllThreads.Num() > 0)
	{
		return; // already initialized
	}

	int32 NumWorkers = GetDefault<UImgMediaSettings>()->CacheThreads;

	if (NumWorkers <= 0)
	{
		NumWorkers = FPlatformMisc::NumberOfWorkerThreadsToSpawn();
	}

	uint32 StackSize = GetWorkerStackSize();

	// create worker threads
	FScopeLock Lock(&CriticalSection);

	while (NumWorkers-- > 0)
	{
		CreateWorker(StackSize);
	}

#if WITH_EDITOR
	UpdateSettingsHandle = UImgMediaSettings::OnSettingsChanged().AddRaw(this, &FImgMediaScheduler::UpdateSettings);
#endif
}


uint32 FImgMediaScheduler::GetWorkerStackSize()
{
	uint32 StackSize = GetDefault<UImgMediaSettings>()->CacheThreadStackSizeKB;

	if (StackSize < IMGMEDIA_SCHEDULERTHREAD_MIN_STACK_SIZE)
	{
		StackSize = IMGMEDIA_SCHEDULERTHREAD_MIN_STACK_SIZE;
	}

	return StackSize;
}


void FImgMediaScheduler::CreateWorker(uint32 StackSize)
{
	FImgMediaSchedulerThread* Thread = new FImgMediaSchedulerThread(*this, StackSize, TPri_Normal);
	AvailableThreads.Add(Thread);
	AllThreads.Add(Thread);
}


void FImgMediaScheduler::RegisterLoader(const TSharedRef<FImgMediaLoader, ESPMode::ThreadSafe>& Loader)
{
	FScopeLock Lock(&CriticalSection);
	Loaders.Add(Loader);
}


void FImgMediaScheduler::Shutdown()
{
	FScopeLock Lock(&CriticalSection);

#if WITH_EDITOR
	UImgMediaSettings::OnSettingsChanged().Remove(UpdateSettingsHandle);
#endif

	// destroy worker threads
	for (FImgMediaSchedulerThread* Thread : AllThreads)
	{
		delete Thread;
	}

	AllThreads.Empty();
	AvailableThreads.Empty();

	LoaderRoundRobin = INDEX_NONE;
}


void FImgMediaScheduler::UnregisterLoader(const TSharedRef<FImgMediaLoader, ESPMode::ThreadSafe>& Loader)
{
	FScopeLock Lock(&CriticalSection);

	const int32 LoaderIndex = Loaders.Find(Loader);
	RemoveLoader(LoaderIndex);
}


void FImgMediaScheduler::RemoveLoader(int32 LoaderIndex)
{
	if (Loaders.IsValidIndex(LoaderIndex))
	{
		Loaders.RemoveAt(LoaderIndex);

		if (LoaderIndex <= LoaderRoundRobin)
		{
			--LoaderRoundRobin;
			check(LoaderRoundRobin >= INDEX_NONE);
		}
	}
}


/* IMediaClockSink interface
* 
* Note that this is ticked both from its own registration as clock sink with MFW,
* but also explicitly at the end of any ImgMediaPlayer tick. The latter is done to
* ensure we schedule jobs also when waiting for blocked playback to resolve.
 *****************************************************************************/

void FImgMediaScheduler::TickInput(FTimespan DeltaTime, FTimespan Timecode)
{
	FScopeLock Lock(&CriticalSection);

	while (AvailableThreads.Num() > 0)
	{
		IQueuedWork* Work = GetWorkOrReturnToPool(nullptr);

		if (Work == nullptr)
		{
			break;
		}

		FImgMediaSchedulerThread* Thread = AvailableThreads.Pop();
		check(Thread != nullptr);

		Thread->QueueWork(Work);
	}
}


/* IImgMediaScheduler interface
 *****************************************************************************/

IQueuedWork* FImgMediaScheduler::GetWorkOrReturnToPool(FImgMediaSchedulerThread* Thread)
{
#if WITH_EDITOR
	bool DeleteThread = false;
#endif
	{
		FScopeLock Lock(&CriticalSection);

		if ((Thread == nullptr) || FPlatformProcess::SupportsMultithreading())
		{
			const int32 NumLoaders = Loaders.Num();

			if (NumLoaders > 0)
			{
				int32 CheckedLoaders = 0;

				while (CheckedLoaders++ < NumLoaders)
				{
					LoaderRoundRobin = (LoaderRoundRobin + 1) % Loaders.Num();

					TSharedPtr<FImgMediaLoader, ESPMode::ThreadSafe> Loader = Loaders[LoaderRoundRobin].Pin();

					if (Loader.IsValid())
					{
						IQueuedWork* Work = Loader->GetWork();

						if (Work != nullptr)
						{
							return Work;
						}
					}
					else
					{
						RemoveLoader(LoaderRoundRobin);
					}
				}
			}
		}

		if (Thread != nullptr)
		{
#if WITH_EDITOR
			// If there are too many workers then delete this one.
			int32 NumWorkers = GetDefault<UImgMediaSettings>()->CacheThreads;
			if (NumWorkers <= 0)
			{
				NumWorkers = FPlatformMisc::NumberOfWorkerThreadsToSpawn();
			}
			if (NumWorkers < AllThreads.Num())
			{
				AllThreads.Remove(Thread);
				DeleteThread = true;
			}
			else
			{
				AvailableThreads.Add(Thread);
			}
#else
			AvailableThreads.Add(Thread);
#endif
		}
	}

#if WITH_EDITOR
	if (DeleteThread)
	{
		delete Thread;
	}
#endif

	return nullptr;
	}


#if WITH_EDITOR

void FImgMediaScheduler::UpdateSettings(const UImgMediaSettings* Settings)
{
	FScopeLock Lock(&CriticalSection);

	// Create more workers if needed.
	uint32 StackSize = GetWorkerStackSize();
	while (Settings->CacheThreads > AllThreads.Num())
	{
		CreateWorker(StackSize);
	}

}

#endif
