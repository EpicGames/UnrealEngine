// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Delegates/IDelegateInstance.h"
#include "HAL/CriticalSection.h"
#include "IImgMediaScheduler.h"
#include "IMediaClockSink.h"
#include "Templates/SharedPointer.h"

class FImgMediaLoader;
class FImgMediaSchedulerThread;
class UImgMediaSettings;


class FImgMediaScheduler
	: public IMediaClockSink
	, protected IImgMediaScheduler
{
public:

	/** Default constructor. */
	FImgMediaScheduler();

	/** Virtual destructor. */
	virtual ~FImgMediaScheduler();

public:

	/**
	 * Initialize the scheduler.
	 *
	 * @see Shutdown
	 */
	void Initialize();

	/**
	 * Register an image sequencer loader.
	 *
	 * @param Loader The loader to register.
	 * @see UnregisterLoader
	 */
	void RegisterLoader(const TSharedRef<FImgMediaLoader, ESPMode::ThreadSafe>& Loader);

	/**
	 * Shut down the scheduler.
	 *
	 * @see Initialize
	 */
	void Shutdown();

	/**
	 * Unregister an image sequence loader.
	 *
	 * @param Loader The loader to unregister.
	 * @see RegisterLoader
	 */
	void UnregisterLoader(const TSharedRef<FImgMediaLoader, ESPMode::ThreadSafe>& Loader);

public:

	//~ IMediaClockSink interface

	virtual void TickInput(FTimespan DeltaTime, FTimespan Timecode) override;

protected:

	/**
	 * Remove the loader with the specified index.
	 *
	 * @param LoaderIndex Index of the loader to remove.
	 */
	void RemoveLoader(int32 LoaderIndex);

protected:

	//~ IImgMediaScheduler interface

	virtual IQueuedWork* GetWorkOrReturnToPool(FImgMediaSchedulerThread* Thread) override;

private:

	/** Collection of all worker threads. */
	TArray<FImgMediaSchedulerThread*> AllThreads;

	/** Threads that are available to perform work. */
	TArray<FImgMediaSchedulerThread*> AvailableThreads;

	/** Critical section for locking access to loader collection. */
	FCriticalSection CriticalSection;

	/** Collection of registered image sequence loaders. */
	TArray<TWeakPtr<FImgMediaLoader, ESPMode::ThreadSafe>> Loaders;

	/** Index of the last loader that provided a work item. */
	int32 LoaderRoundRobin;

	uint32 GetWorkerStackSize();
	void CreateWorker(uint32 StackSize);

#if WITH_EDITOR

	FDelegateHandle UpdateSettingsHandle;
	/**
	 * Updates our settings from ImgMediaAsettings.
	 */
	void UpdateSettings(const UImgMediaSettings* Settings);

#endif
};
