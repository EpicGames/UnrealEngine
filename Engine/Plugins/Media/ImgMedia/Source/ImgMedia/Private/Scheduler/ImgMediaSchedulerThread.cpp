// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImgMediaSchedulerThread.h"

#include "HAL/Event.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "HAL/RunnableThread.h"
#include "Misc/IQueuedWork.h"

#include "IImgMediaScheduler.h"


/* FImgMediaSchedulerThread structors
 *****************************************************************************/

FImgMediaSchedulerThread::FImgMediaSchedulerThread(IImgMediaScheduler& InOwner, uint32 StackSize, EThreadPriority ThreadPriority)
	: Owner(InOwner)
	, QueuedWork(nullptr)
	, Stopping(false)
{
	WorkEvent = FPlatformProcess::GetSynchEventFromPool();
	check(WorkEvent != nullptr);

	static int32 ThreadIndex = 0;
	const FString ThreadName = FString::Printf(TEXT("FImgMediaSchedulerThread %d"), ThreadIndex++);

	Thread = FRunnableThread::Create(this, *ThreadName, StackSize, ThreadPriority, FPlatformAffinity::GetPoolThreadMask());
	check(Thread != nullptr);
}


FImgMediaSchedulerThread::~FImgMediaSchedulerThread()
{
	Stopping = true;
	WorkEvent->Trigger();
	
	Thread->WaitForCompletion();
	Thread->Kill(true);
	delete Thread;
	Thread = nullptr;

	FPlatformProcess::ReturnSynchEventToPool(WorkEvent);
	WorkEvent = nullptr;
}


/* FImgMediaSchedulerThread interface
 *****************************************************************************/

void FImgMediaSchedulerThread::QueueWork(IQueuedWork* Work)
{
	check(QueuedWork == nullptr);

	QueuedWork = Work;
	FPlatformMisc::MemoryBarrier();

	WorkEvent->Trigger();
}


/* FRunnable interface
 *****************************************************************************/

FSingleThreadRunnable* FImgMediaSchedulerThread::GetSingleThreadInterface()
{
	return this;
}


uint32 FImgMediaSchedulerThread::Run()
{
	while (!Stopping.Load(EMemoryOrder::Relaxed))
	{
		if (WorkEvent->Wait(10))
		{
			// consume the queued work item
			IQueuedWork* Work = QueuedWork;
			QueuedWork = nullptr;
			FPlatformMisc::MemoryBarrier();

			// if there was no queued work then we must be stopping
			check((Work != nullptr) || Stopping.Load(EMemoryOrder::Relaxed));

			// perform work until there is no more
			while (Work != nullptr)
			{
				Work->DoThreadedWork();
				Work = Owner.GetWorkOrReturnToPool(this);
			}
		}
	}

	return 0;
}


/* FSingleThreadRunnable interface
 *****************************************************************************/

void FImgMediaSchedulerThread::Tick()
{
	IQueuedWork* Work = QueuedWork;
	QueuedWork = nullptr;

	while (Work != nullptr)
	{
		Work->DoThreadedWork();
		Work = Owner.GetWorkOrReturnToPool(this);
	}
}