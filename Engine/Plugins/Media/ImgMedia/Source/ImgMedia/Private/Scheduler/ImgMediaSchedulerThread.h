// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformAffinity.h"
#include "HAL/Runnable.h"
#include "Misc/SingleThreadRunnable.h"
#include "Templates/Atomic.h"

class FEvent;
class FRunnableThread;
class IImgMediaScheduler;
class IQueuedWork;


class FImgMediaSchedulerThread
	: public FRunnable
	, private FSingleThreadRunnable
{
public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InOwner The scheduler that owns this thread.
	 */
	FImgMediaSchedulerThread(IImgMediaScheduler& InOwner, uint32 StackSize, EThreadPriority ThreadPriority);

	/** Virtual destructor. */
	virtual ~FImgMediaSchedulerThread();

public:

	/**
	 * Queue the specified work item.
	 *
	 * This method is called by the scheduler on threads that currently do not perform any work.
	 *
	 * @param Work The work item to queue.
	 */
	void QueueWork(IQueuedWork* Work);

public:

	//~ FRunnable interface

	virtual FSingleThreadRunnable* GetSingleThreadInterface() override;
	virtual uint32 Run() override;

protected:

	//~ FSingleThreadRunnable interface

	virtual void Tick() override;

private:

	/** The scheduler that owns this thread. */
	IImgMediaScheduler& Owner;

	/** Queued work item to be processed by this thread. */
	IQueuedWork* QueuedWork;

	/** Whether this thread is stopping. */
	TAtomic<bool> Stopping;

	/** The thread object. */
	FRunnableThread* Thread;

	/** Event signaling that work is available. */
	FEvent* WorkEvent;
};