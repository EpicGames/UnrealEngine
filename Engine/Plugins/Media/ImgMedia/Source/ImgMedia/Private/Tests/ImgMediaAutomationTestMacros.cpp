// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImgMediaAutomationTestMacros.h"
#include "Misc/AssertionMacros.h"
#include "Logging/LogCategory.h"

DEFINE_LOG_CATEGORY(LogImgMediaAutomationTest);

#if !UE_BUILD_SHIPPING
extern CORE_API bool GIgnoreDebugger;
#endif

FImgMediaTestEnsureScope::FImgMediaTestEnsureScope()
	: FEnsureScope(
		[](const FEnsureHandlerArgs& Args)
			{
				UE_LOGF(LogImgMediaAutomationTest, Display, "Ensure condition failed: %s\n%ls\n", Args.Expression, Args.Message);
				return true;
			}
	)
#if !UE_BUILD_SHIPPING
	, bIgnoreDebugger(GIgnoreDebugger)
#else
	, bIgnoreDebugger(false)
#endif
{
#if !UE_BUILD_SHIPPING
	GIgnoreDebugger = true;
#endif
}

FImgMediaTestEnsureScope::~FImgMediaTestEnsureScope()
{
#if !UE_BUILD_SHIPPING
	GIgnoreDebugger = bIgnoreDebugger;
#endif
}