// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Tests/EnsureScope.h"

DECLARE_LOG_CATEGORY_EXTERN(LogImgMediaAutomationTest, Display, All)

/**
 * Version of EnsureScope that ignores the debugger
 * to make it possible to run the tests with debugger attached and not 
 * have it break on ensures. It will log the ensure message to leave a trace.
 */
class FImgMediaTestEnsureScope : public FEnsureScope
{
public:
	FImgMediaTestEnsureScope();
	~FImgMediaTestEnsureScope();

private:
	bool bIgnoreDebugger;
};
