// Copyright Epic Games, Inc. All Rights Reserved.

#include "Assets/MediaTileSelection.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/FileManager.h"
#include "ImgMediaAutomationTestMacros.h"
#include "ImgMediaSource.h"
#include "Loader/ImgMediaLoaderUtils.h"
#include "Logging/LogMacros.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FImgMediaTests, "System.Plugins.ImgMedia.TileSelection", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FImgMediaTests::RunTest(const FString& Parameters)
{
	{
		// Single continuous region
		FMediaTileSelection Selection = FMediaTileSelection(10, 10);
		for (int32 Ty = 3; Ty < 7; Ty++)
		{
			for (int32 Tx = 1; Tx < 6; Tx++)
			{
				Selection.SetVisible(Tx, Ty);
			}
		}

		FIntRect R0 = Selection.GetVisibleRegion();
		FIntRect R1 = Selection.GetVisibleRegions()[0];
		AddErrorIfFalse(R0 == R1, TEXT("FImgMediaTests: Mismatched tile regions."));
	}

	{
		//Two regions on both sides
		FMediaTileSelection Selection = FMediaTileSelection(10, 10);
		for (int32 Ty = 0; Ty < 10; Ty++)
		{
			for (int32 Tx = 0; Tx < 2; Tx++)
			{
				Selection.SetVisible(Tx, Ty);
			}

			for (int32 Tx = 8; Tx < 10; Tx++)
			{
				Selection.SetVisible(Tx, Ty);
			}
		}
		TArray<FIntRect> Result = Selection.GetVisibleRegions();

		bool bExpectedNum = Result.Num() == 2;
		if (bExpectedNum)
		{
			AddErrorIfFalse(Result[0] == FIntRect(FIntPoint(0, 0), FIntPoint(2, 10)), TEXT("FImgMediaTests: Mismatched tile regions."));
			AddErrorIfFalse(Result[1] == FIntRect(FIntPoint(8, 0), FIntPoint(10, 10)), TEXT("FImgMediaTests: Mismatched tile regions."));
		}
		else
		{
			AddError(TEXT("FImgMediaTests: Expected 2 regions."));
		}
	}

	{
		//Each row has a different length, resulting in one region for each row with the current algo.
		FMediaTileSelection Selection = FMediaTileSelection(10, 10);
		for (int32 Ty = 0; Ty < 10; Ty++)
		{
			for (int32 Tx = 0; Tx < Ty + 1; Tx++)
			{
				Selection.SetVisible(Tx, Ty);
			}
		}

		TArray<FIntRect> Result = Selection.GetVisibleRegions();

		AddErrorIfFalse(Result.Num() == 10, TEXT("FImgMediaTests: Expected 10 regions."));
	}

	{
		//Worst case: checkerboard pattern where each tile becomes a region.
		FMediaTileSelection Selection = FMediaTileSelection(4, 4);
		Selection.SetVisible(0, 0); Selection.SetVisible(2, 0);
		Selection.SetVisible(1, 1); Selection.SetVisible(3, 1);
		Selection.SetVisible(0, 2); Selection.SetVisible(2, 2);
		Selection.SetVisible(1, 3); Selection.SetVisible(3, 3);

		TArray<FIntRect> Result = Selection.GetVisibleRegions();

		AddErrorIfFalse(Result.Num() == 8, TEXT("FImgMediaTests: Expected 8 regions."));
	}

	// Out-of-range positive coords clamp to the corner tile rather than indexing past
	// the bit array. The ensure fires (expected) - we only verify the safe-clamp behavior.
	{
		FMediaTileSelection Sel(4, 4);
		{
			FImgMediaTestEnsureScope Scope;
			Sel.SetVisible(100, 100);
		}
		TestTrue(TEXT("out-of-range positive clamps to corner (3,3)"), Sel.IsVisible(3, 3));
	}

	// Negative coords clamp to (0,0) rather than producing a negative index.
	{
		FMediaTileSelection Sel(4, 4);
		{
			FImgMediaTestEnsureScope Scope;
			Sel.SetVisible(-5, -5);
		}
		TestTrue(TEXT("negative coords clamp to (0,0)"), Sel.IsVisible(0, 0));
	}

	// Contains returns false on dimension mismatch (cross-dim comparison is meaningless).
	{
		FMediaTileSelection A(4, 4);
		A.SetVisible(0, 0);

		FMediaTileSelection B(8, 8);
		B.SetVisible(0, 0);

		{
			FImgMediaTestEnsureScope Scope;
			TestFalse(TEXT("Contains returns false on dimension mismatch"), A.Contains(B));
		}
	}
	
	// Extreme mip levels must not invoke shift-of-int UB; result collapses to 1x1.
	{
		const FMediaTileSelection Sel = FMediaTileSelection::CreateForTargetMipLevel(
			FIntPoint(1024, 1024), FIntPoint(256, 256), 31, false);
		TestEqual(TEXT("mip level 31 produces 1x1"), Sel.GetDimensions(), FIntPoint(1, 1));
	}
	{
		const FMediaTileSelection Sel = FMediaTileSelection::CreateForTargetMipLevel(
			FIntPoint(1024, 1024), FIntPoint(256, 256), 100, false);
		TestEqual(TEXT("mip level 100 produces 1x1"), Sel.GetDimensions(), FIntPoint(1, 1));
	}

	// Reasonable mid-range mip levels keep working.
	{
		const FMediaTileSelection Sel = FMediaTileSelection::CreateForTargetMipLevel(
			FIntPoint(1024, 1024), FIntPoint(256, 256), 1, false);
		TestEqual(TEXT("mip level 1 of 4x4 produces 2x2"), Sel.GetDimensions(), FIntPoint(2, 2));
	}

	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FImgMediaLoaderUtilsTests, "System.Plugins.ImgMedia.LoaderUtils", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FImgMediaLoaderUtilsTests::RunTest(const FString& Parameters)
{
	TArray<FFrameRate> FrameRates =
	{
	{60000, 1001},	// 59.95
	{30000, 1001},	// 29.97
	{24000, 1001},	// 23.976
	{60, 1},			// 60
	{30, 1},			// 30
	{50, 1},			// 50
	{25, 1},			// 25
	{24, 1}			// 24
	};


	// Test to ensure reversibility of Frame to Time conversion functions
	// This is necessary for frame accuracy and consistency with the sequencer.
	for (FFrameRate FrameRate : FrameRates)
	{
		for (uint32 FrameNumber = 0; FrameNumber < 100; FrameNumber++)
		{
			{
				FTimespan StartTime = UE::ImgMediaLoaderUtils::GetFrameStartTime(FrameNumber, FrameRate);
				uint32 ConvertedFrameNumber = UE::ImgMediaLoaderUtils::TimeToFrameNumber(StartTime, FrameRate);			
				AddErrorIfFalse(ConvertedFrameNumber == FrameNumber, FString::Printf(TEXT("FImgMediaLoaderUtilsTests: Frame Start Time %lld conversion at %f not reversible: Frame %i != %i"), StartTime.GetTicks(), FrameRate.AsDecimal(), FrameNumber, ConvertedFrameNumber));
				int64 ConvertedFrameNumber64 = UE::ImgMediaLoaderUtils::TimeToFrameNumberUnbound(StartTime, FrameRate);
				AddErrorIfFalse(ConvertedFrameNumber64 == static_cast<int64>(FrameNumber), FString::Printf(TEXT("FImgMediaLoaderUtilsTests: Unbound Frame Start Time %lld conversion at %f not reversible: Frame %i != %lld"), StartTime.GetTicks(), FrameRate.AsDecimal(), FrameNumber, ConvertedFrameNumber64));
			}
			{
				// Testing that GetFrameStartTime is indeed the first tick of the frame.
				// One tick before the start of next frame should be the previous frame.
				FTimespan EndTime = UE::ImgMediaLoaderUtils::GetFrameStartTime(FrameNumber + 1, FrameRate) - FTimespan(1);
				uint32 ConvertedFrameNumber = UE::ImgMediaLoaderUtils::TimeToFrameNumber(EndTime, FrameRate);
				AddErrorIfFalse(ConvertedFrameNumber == FrameNumber, FString::Printf(TEXT("FImgMediaLoaderUtilsTests: Frame End Time %lld conversion at %f not reversible: Frame %i != %i"), EndTime.GetTicks(), FrameRate.AsDecimal(), FrameNumber, ConvertedFrameNumber));
				int64 ConvertedFrameNumber64 = UE::ImgMediaLoaderUtils::TimeToFrameNumberUnbound(EndTime, FrameRate);
				AddErrorIfFalse(ConvertedFrameNumber64 == static_cast<int64>(FrameNumber), FString::Printf(TEXT("FImgMediaLoaderUtilsTests: Unbound Frame End Time %lld conversion at %f not reversible: Frame %i != %lld"), EndTime.GetTicks(), FrameRate.AsDecimal(), FrameNumber, ConvertedFrameNumber64));
			}
		}
	}
	
	return !HasAnyErrors();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FImgMediaSanitizePathTests, "System.Plugins.ImgMedia.SanitizePath", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FImgMediaSanitizePathTests::RunTest(const FString& Parameters)
{
	// Setup the test, needs some existing files in different folders
	TArray<FString> CreatedDirectories;
	TArray<FString> CreatedFiles;
	
	auto MakeDirectory = [&CreatedDirectories](const FString& InPath) -> bool
	{
		if (FPaths::DirectoryExists(InPath))
		{
			return true;
		}
		if (IFileManager::Get().MakeDirectory(*InPath))
		{
			CreatedDirectories.Add(InPath);
			return true;
		}
		return false;
	};

	auto MakeFile = [&CreatedFiles](const FString& InFilepath) -> bool
	{
		if (FPaths::FileExists(InFilepath))
		{
			return true;
		}
		const TUniquePtr<FArchive> FileWriter(IFileManager::Get().CreateFileWriter(*InFilepath));
		if (FileWriter)
		{
			int32 value = 1234;
			(*FileWriter) << value;
			CreatedFiles.Add(InFilepath);
			return true;
		}
		return false;
	};

	{
		const FString ContentMoviesDirectory = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectContentDir(), TEXT("Movies")));
		const FString ContentSequenceDirectory = FPaths::Combine(ContentMoviesDirectory, TEXT("ImgMediaTestSequence"));
		const FString ContentSequenceImgPath = FPaths::Combine(ContentSequenceDirectory, TEXT("Image0000.exr"));
		MakeDirectory(ContentMoviesDirectory);
		MakeDirectory(ContentSequenceDirectory);
		MakeFile(ContentSequenceImgPath);
	}
	{
		const FString ProjectMoviesDirectory = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectDir(), TEXT("Movies")));
		const FString ProjectSequenceDirectory = FPaths::Combine(ProjectMoviesDirectory, TEXT("ImgMediaTestSequence"));
		const FString ProjectSequenceImgPath = FPaths::Combine(ProjectSequenceDirectory, TEXT("Image0000.exr"));
		MakeDirectory(ProjectMoviesDirectory);
		MakeDirectory(ProjectSequenceDirectory);
		MakeFile(ProjectSequenceImgPath);
	}
	{
		const FString EngineMoviesDirectory = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::EngineDir(), TEXT("Movies")));
		const FString EngineSequenceDirectory = FPaths::Combine(EngineMoviesDirectory, TEXT("ImgMediaTestSequence"));
		const FString EngineSequenceImgPath = FPaths::Combine(EngineSequenceDirectory, TEXT("Image0000.exr"));
		MakeDirectory(EngineMoviesDirectory);
		MakeDirectory(EngineSequenceDirectory);
		MakeFile(EngineSequenceImgPath);
	}
	
	ON_SCOPE_EXIT
	{
		for (const FString& File : CreatedFiles)
		{
			IFileManager::Get().Delete(*File);
		}
		
		for(const FString& Directory : ReverseIterate(CreatedDirectories))
		{
			IFileManager::Get().DeleteDirectory(*Directory);
		}
	};

	// ---- Tests Begin here

	auto TestSanitizeTokenizedPath = [this](const FString& InPath, const FString& InExpectedResult, FString& OutError)
	{
		const FString SanitizedPath = UImgMediaSource::SanitizeTokenizedSequencePath(InPath);
		if (SanitizedPath != InExpectedResult)
		{
			OutError = FString::Printf(TEXT("SanitizeTokenizedSequencePath failed: Input: \"%s\" Result: \"%s\" Expected \"%s\""), *InPath, *SanitizedPath, *InExpectedResult);
			return false;
		}
		return true;
	};

	FString Error;
	
	// Empty path
	AddErrorIfFalse(TestSanitizeTokenizedPath(TEXT(""), TEXT(""), Error), Error);

	// Already sanitized
	AddErrorIfFalse(TestSanitizeTokenizedPath(TEXT("./Movies/ImgMediaTestSequence"), TEXT("./Movies/ImgMediaTestSequence"), Error), Error);
	AddErrorIfFalse(TestSanitizeTokenizedPath(TEXT("./Movies/ImgMediaTestSequence/"), TEXT("./Movies/ImgMediaTestSequence"), Error), Error);
	AddErrorIfFalse(TestSanitizeTokenizedPath(TEXT(".\\Movies\\ImgMediaTestSequence"), TEXT("./Movies/ImgMediaTestSequence"), Error), Error);
	AddErrorIfFalse(TestSanitizeTokenizedPath(TEXT(".\\Movies\\ImgMediaTestSequence\\"), TEXT("./Movies/ImgMediaTestSequence"), Error), Error);
	AddErrorIfFalse(TestSanitizeTokenizedPath(TEXT("\".\\Movies\\ImgMediaTestSequence\""), TEXT("./Movies/ImgMediaTestSequence"), Error), Error);

	// With the filename (it should remove it)
	AddErrorIfFalse(TestSanitizeTokenizedPath(TEXT("./Movies/ImgMediaTestSequence/Image0000.exr"), TEXT("./Movies/ImgMediaTestSequence"), Error), Error);
	AddErrorIfFalse(TestSanitizeTokenizedPath(TEXT(".\\Movies\\ImgMediaTestSequence\\Image0000.exr"), TEXT("./Movies/ImgMediaTestSequence"), Error), Error);
	AddErrorIfFalse(TestSanitizeTokenizedPath(TEXT("\".\\Movies\\ImgMediaTestSequence\\Image0000.exr\""), TEXT("./Movies/ImgMediaTestSequence"), Error), Error);
	
	{
		// Project relative
		FString ProjectRelative = FPaths::Combine(FPaths::ProjectDir(), TEXT("Movies/ImgMediaTestSequence"));
		AddErrorIfFalse(TestSanitizeTokenizedPath(ProjectRelative, TEXT("./Movies/ImgMediaTestSequence"), Error), Error);
		AddErrorIfFalse(TestSanitizeTokenizedPath(FPaths::ConvertRelativePathToFull(ProjectRelative), TEXT("./Movies/ImgMediaTestSequence"), Error), Error);

		// With filename
		ProjectRelative = FPaths::Combine(ProjectRelative, TEXT("Image0000.exr"));
		AddErrorIfFalse(TestSanitizeTokenizedPath(ProjectRelative, TEXT("./Movies/ImgMediaTestSequence"), Error), Error);
		AddErrorIfFalse(TestSanitizeTokenizedPath(FPaths::ConvertRelativePathToFull(ProjectRelative), TEXT("./Movies/ImgMediaTestSequence"), Error), Error);
	}

	{
		// Content relative
		FString ContentRelative = FPaths::Combine(FPaths::ProjectContentDir(), TEXT("Movies/ImgMediaTestSequence"));
		AddErrorIfFalse(TestSanitizeTokenizedPath(ContentRelative, TEXT("./Movies/ImgMediaTestSequence"), Error), Error);
		AddErrorIfFalse(TestSanitizeTokenizedPath(FPaths::ConvertRelativePathToFull(ContentRelative), TEXT("./Movies/ImgMediaTestSequence"), Error), Error);

		// With filename
		ContentRelative = FPaths::Combine(ContentRelative, TEXT("Image0000.exr"));
		AddErrorIfFalse(TestSanitizeTokenizedPath(ContentRelative, TEXT("./Movies/ImgMediaTestSequence"), Error), Error);
		AddErrorIfFalse(TestSanitizeTokenizedPath(FPaths::ConvertRelativePathToFull(ContentRelative), TEXT("./Movies/ImgMediaTestSequence"), Error), Error);
	}

	{
		// Outside of project (ex: engine), expect absolute path
		FString EngineRelative = FPaths::Combine(FPaths::EngineDir(), TEXT("Movies/ImgMediaTestSequence"));
		FString EngineRelativeFull = FPaths::ConvertRelativePathToFull(EngineRelative);	// expected
		AddErrorIfFalse(TestSanitizeTokenizedPath(EngineRelative, EngineRelativeFull, Error), Error);
		AddErrorIfFalse(TestSanitizeTokenizedPath(FPaths::ConvertRelativePathToFull(EngineRelative), EngineRelativeFull, Error), Error);

		// With filename
		EngineRelative = FPaths::Combine(EngineRelative,  TEXT("Image0000.exr"));
		AddErrorIfFalse(TestSanitizeTokenizedPath(EngineRelative, EngineRelativeFull, Error), Error);
		AddErrorIfFalse(TestSanitizeTokenizedPath(FPaths::ConvertRelativePathToFull(EngineRelative), EngineRelativeFull, Error), Error);
	}

	{
		// Project relative with Token
		FString Expected = TEXT("{project_dir}/Movies/ImgMediaTestSequence");
		AddErrorIfFalse(TestSanitizeTokenizedPath(Expected, Expected, Error), Error);
		AddErrorIfFalse(TestSanitizeTokenizedPath(TEXT("{project_dir}/Movies/ImgMediaTestSequence/Image0000.exr"), Expected, Error), Error);
	}

	{
		// Engine relative with Token
		FString Expected = TEXT("{engine_dir}/Movies/ImgMediaTestSequence");
		AddErrorIfFalse(TestSanitizeTokenizedPath(Expected, Expected, Error), Error);
		AddErrorIfFalse(TestSanitizeTokenizedPath(TEXT("{engine_dir}/Movies/ImgMediaTestSequence/Image0000.exr"), Expected, Error), Error);
	}
	
	return !HasAnyErrors();
}

#endif
