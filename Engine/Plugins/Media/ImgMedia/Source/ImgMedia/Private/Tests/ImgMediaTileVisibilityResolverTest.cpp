// Copyright Epic Games, Inc. All Rights Reserved.

#include "Assets/ImgMediaTileVisibilityResolver.h"
#include "Assets/MediaTileVisibility.h"
#include "Assets/Providers/ImgMediaProviderUtils.h"
#include "Assets/Providers/ImgMediaSphereVisibilityProvider.h"
#include "Async/Async.h"
#include "Async/Future.h"
#include "CoreTypes.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "ImgMediaAutomationTestMacros.h"
#include "MediaTexture.h"
#include "MediaTextureTracker.h"
#include "Misc/AutomationTest.h"
#include "UObject/StrongObjectPtr.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace ImgMediaTileVisibilityTest
{
	class FStubProvider : public FMediaTileVisibilityProvider
	{
	public:
		FStubProvider(int32 InMipToMark, int32 InTileX, int32 InTileY,
			int32 InNumTilesX, int32 InNumTilesY, int32 InUpscale = -1)
			: MipToMark(InMipToMark), TileX(InTileX), TileY(InTileY)
			, NumTilesX(InNumTilesX), NumTilesY(InNumTilesY), Upscale(InUpscale)
		{}

		virtual void GatherVisibleTiles(const FMediaSequenceInfo&,
			FMediaTileVisibilityRequest& OutRequest) const override
		{
			FMediaTileSelection Sel(NumTilesX, NumTilesY);
			Sel.SetVisible(TileX, TileY);
			OutRequest.VisibleTiles.Add(MipToMark, MoveTemp(Sel));
			if (Upscale >= 0)
			{
				OutRequest.MipLevelToUpscale = Upscale;
			}
		}

		virtual bool IsAlive() const override { return bAlive; }

		bool bAlive = true;
		int32 MipToMark, TileX, TileY, NumTilesX, NumTilesY, Upscale;
	};

	// Provider that blocks inside GatherVisibleTiles until the test releases it,
	// to deterministically reproduce "worker mid-build, game thread acts on resolver".
	class FBlockingStubProvider : public FMediaTileVisibilityProvider
	{
	public:
		FBlockingStubProvider()
			: InsideEvent(FPlatformProcess::GetSynchEventFromPool(true))
			, ReleaseEvent(FPlatformProcess::GetSynchEventFromPool(true))
		{}
		~FBlockingStubProvider()
		{
			FPlatformProcess::ReturnSynchEventToPool(InsideEvent);
			FPlatformProcess::ReturnSynchEventToPool(ReleaseEvent);
		}
		FBlockingStubProvider(const FBlockingStubProvider&) = delete;
		FBlockingStubProvider& operator=(const FBlockingStubProvider&) = delete;

		virtual void GatherVisibleTiles(const FMediaSequenceInfo&,
			FMediaTileVisibilityRequest& OutRequest) const override
		{
			InsideEvent->Trigger();
			ReleaseEvent->Wait();
			FMediaTileSelection Sel(4, 4);
			Sel.SetVisible(0, 0);
			OutRequest.VisibleTiles.Add(0, MoveTemp(Sel));
		}
		virtual bool IsAlive() const override { return true; }

		FEvent* InsideEvent;
		FEvent* ReleaseEvent;
	};

	static FMediaSequenceInfo MakeTestSequenceInfo()
	{
		FMediaSequenceInfo Info;
		Info.Dim = FIntPoint(1024, 1024);
		Info.NumMipLevels = 1;
		Info.TilingDescription.TileSize = FIntPoint(256, 256);
		Info.TilingDescription.TileNum = FIntPoint(4, 4);
		return Info;
	}
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMediaTileVisibilityRequestMergeTest,
	"System.Plugins.ImgMedia.TileVisibility.RequestMerge",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMediaTileVisibilityRequestMergeTest::RunTest(const FString&)
{
	FMediaTileVisibilityRequest A;
	FMediaTileSelection SelA(4, 4);
	SelA.SetVisible(0, 0);
	A.VisibleTiles.Add(0, SelA);
	A.MipLevelToUpscale = 3;

	FMediaTileVisibilityRequest B;
	FMediaTileSelection SelB(4, 4);
	SelB.SetVisible(3, 3);
	B.VisibleTiles.Add(0, SelB);
	B.MipLevelToUpscale = 1;

	A.Merge(B);

	TestEqual(TEXT("merged mip count"), A.VisibleTiles.Num(), 1);
	TestTrue(TEXT("OR'd tile (0,0) preserved"), A.VisibleTiles[0].IsVisible(0, 0));
	TestTrue(TEXT("OR'd tile (3,3) added"), A.VisibleTiles[0].IsVisible(3, 3));
	TestEqual(TEXT("min non-negative upscale wins"), A.MipLevelToUpscale, 1);

	// Adding only the disabled (-1) upscale must not clobber an existing valid value.
	FMediaTileVisibilityRequest C;
	C.MipLevelToUpscale = -1;
	A.Merge(C);
	TestEqual(TEXT("disabled upscale does not clobber"), A.MipLevelToUpscale, 1);

	// New mip level merges as a new entry.
	FMediaTileVisibilityRequest D;
	FMediaTileSelection SelD(2, 2);
	SelD.SetVisible(0, 1);
	D.VisibleTiles.Add(1, SelD);
	A.Merge(D);
	TestEqual(TEXT("new mip added"), A.VisibleTiles.Num(), 2);
	TestTrue(TEXT("new mip tile present"), A.VisibleTiles[1].IsVisible(0, 1));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMediaTileVisibilityRequestResetTest,
	"System.Plugins.ImgMedia.TileVisibility.RequestReset",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMediaTileVisibilityRequestResetTest::RunTest(const FString&)
{
	FMediaTileVisibilityRequest A;
	FMediaTileSelection Sel(4, 4);
	Sel.SetVisible(0, 0);
	A.VisibleTiles.Add(0, MoveTemp(Sel));
	A.MipLevelToUpscale = 3;

	A.Reset();

	TestEqual(TEXT("VisibleTiles cleared by Reset"), A.VisibleTiles.Num(), 0);
	TestEqual(TEXT("MipLevelToUpscale reset to -1"), A.MipLevelToUpscale, -1);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMediaTileVisibilityRequestDimensionMismatchTest,
	"System.Plugins.ImgMedia.TileVisibility.RequestDimensionMismatch",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMediaTileVisibilityRequestDimensionMismatchTest::RunTest(const FString&)
{
	// Existing smaller, Other larger: keep the larger to avoid silently dropping coverage.
	{
		FMediaTileVisibilityRequest A;
		FMediaTileSelection SelA(4, 4);
		SelA.SetVisible(0, 0);
		A.VisibleTiles.Add(0, MoveTemp(SelA));

		FMediaTileVisibilityRequest B;
		FMediaTileSelection SelB(8, 8);
		SelB.SetVisible(7, 7);
		B.VisibleTiles.Add(0, MoveTemp(SelB));

		{
			FImgMediaTestEnsureScope Scope;
			A.Merge(B);
		}

		TestEqual(TEXT("dim-mismatch keeps larger 8x8 dimensions"),
			A.VisibleTiles[0].GetDimensions(), FIntPoint(8, 8));
		TestTrue(TEXT("larger selection's tile preserved"),
			A.VisibleTiles[0].IsVisible(7, 7));
	}

	// Existing larger, Other smaller: existing wins; do not regress to smaller dims.
	{
		FMediaTileVisibilityRequest A;
		FMediaTileSelection SelA(8, 8);
		SelA.SetVisible(7, 7);
		A.VisibleTiles.Add(0, MoveTemp(SelA));

		FMediaTileVisibilityRequest B;
		FMediaTileSelection SelB(4, 4);
		SelB.SetVisible(0, 0);
		B.VisibleTiles.Add(0, MoveTemp(SelB));

		{
			FImgMediaTestEnsureScope Scope;
			A.Merge(B);
		}

		TestEqual(TEXT("larger existing 8x8 dimensions kept"),
			A.VisibleTiles[0].GetDimensions(), FIntPoint(8, 8));
		TestTrue(TEXT("existing tile (7,7) preserved"),
			A.VisibleTiles[0].IsVisible(7, 7));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FImgMediaTileVisibilityResolverTest,
	"System.Plugins.ImgMedia.TileVisibility.Resolver",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FImgMediaTileVisibilityResolverTest::RunTest(const FString&)
{
	using namespace ImgMediaTileVisibilityTest;

	FMediaSequenceInfo Info;
	Info.Dim = FIntPoint(1024, 1024);
	Info.NumMipLevels = 1;
	Info.TilingDescription.TileSize = FIntPoint(256, 256);
	Info.TilingDescription.TileNum = FIntPoint(4, 4);

	FImgMediaTileVisibilityResolver Resolver;
	Resolver.SetSequenceInfo(Info);

	auto TestGetSnapshot = [this](FImgMediaTileVisibilityResolver& InResolver, const FString& InContext)
		{
			const auto Snapshot = InResolver.GetSnapshot();
			if (!Snapshot)
			{
				AddError(FString::Printf(TEXT("%s: GetSnapshot returned null"), *InContext), 1);
			}
			return Snapshot;
		};
	
	TSharedRef<FStubProvider> P1 = MakeShared<FStubProvider>(0, 1, 1, 4, 4);
	TSharedRef<FStubProvider> P2 = MakeShared<FStubProvider>(0, 2, 2, 4, 4);
	Resolver.RegisterProvider(P1);
	Resolver.RegisterProvider(P2);
	TestEqual(TEXT("two providers registered"), Resolver.NumProviders(), 2);

	if (const TSharedPtr<FMediaTileVisibilityRequest, ESPMode::ThreadSafe> Snap = TestGetSnapshot(Resolver, TEXT("Two Providers")))
	{
		TestTrue(TEXT("provider 1 contributed (1,1)"), Snap->VisibleTiles[0].IsVisible(1, 1));
		TestTrue(TEXT("provider 2 contributed (2,2)"), Snap->VisibleTiles[0].IsVisible(2, 2));
	}

	// Cached snapshot returns the same data without rebuild.
	if (const TSharedPtr<FMediaTileVisibilityRequest, ESPMode::ThreadSafe> SnapCached = TestGetSnapshot(Resolver, TEXT("Two Providers - Cached")))
	{
		TestTrue(TEXT("cached tile (1,1)"), SnapCached->VisibleTiles[0].IsVisible(1, 1));
	}
	
	// Provider death is observed on next snapshot after invalidation.
	P1->bAlive = false;
	Resolver.InvalidateSnapshot();
	if (const TSharedPtr<FMediaTileVisibilityRequest, ESPMode::ThreadSafe> Snap2 = TestGetSnapshot(Resolver, TEXT("Provider Death")))
	{
		TestFalse(TEXT("dead provider's tile gone"), Snap2->VisibleTiles[0].IsVisible(1, 1));
		TestTrue(TEXT("live provider's tile present"), Snap2->VisibleTiles[0].IsVisible(2, 2));
	}

	TestEqual(TEXT("dead provider was pruned"), Resolver.NumProviders(), 1);

	// Unregister via shared ref.
	Resolver.UnregisterProvider(P2);
	TestEqual(TEXT("no providers after explicit unregister"), Resolver.NumProviders(), 0);
	Resolver.InvalidateSnapshot();
	if (const TSharedPtr<FMediaTileVisibilityRequest, ESPMode::ThreadSafe> Snap3 = TestGetSnapshot(Resolver, TEXT("No Providers")))
	{
		TestEqual(TEXT("snapshot empty after all providers gone"), Snap3->VisibleTiles.Num(), 0);
	}

	// Upscale level: minimum non-negative across providers.
	TSharedRef<FStubProvider> P3 = MakeShared<FStubProvider>(0, 0, 0, 4, 4, /*Upscale=*/5);
	TSharedRef<FStubProvider> P4 = MakeShared<FStubProvider>(0, 0, 0, 4, 4, /*Upscale=*/2);
	Resolver.RegisterProvider(P3);
	Resolver.RegisterProvider(P4);
	Resolver.InvalidateSnapshot();
	TestEqual(TEXT("min non-negative upscale across providers"),
		Resolver.GetMinimumMipLevelToUpscale(), 2);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FImgMediaTileVisibilityResolverConcurrencyTest,
	"System.Plugins.ImgMedia.TileVisibility.Resolver.Concurrency",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FImgMediaTileVisibilityResolverConcurrencyTest::RunTest(const FString&)
{
	using namespace ImgMediaTileVisibilityTest;

	FImgMediaTileVisibilityResolver Resolver;
	Resolver.SetSequenceInfo(MakeTestSequenceInfo());

	TArray<TSharedRef<FStubProvider>> Providers;
	for (int32 i = 0; i < 4; ++i)
	{
		Providers.Add(MakeShared<FStubProvider>(0, i, i, 4, 4));
		Resolver.RegisterProvider(Providers.Last());
	}

	constexpr int32 NumWorkers = 4;
	constexpr int32 IterationsPerWorker = 500;
	TArray<TFuture<int32>> Results;
	Results.Reserve(NumWorkers);

	for (int32 i = 0; i < NumWorkers; ++i)
	{
		Results.Add(Async(EAsyncExecution::Thread, [&Resolver]()
		{
			int32 NullCount = 0;
			for (int32 j = 0; j < IterationsPerWorker; ++j)
			{
				const auto Snap = Resolver.GetSnapshot();
				if (!Snap.IsValid())
				{
					++NullCount;
				}
			}
			return NullCount;
		}));
	}

	for (int32 j = 0; j < IterationsPerWorker; ++j)
	{
		Resolver.InvalidateSnapshot();
	}

	int32 TotalNulls = 0;
	for (TFuture<int32>& F : Results)
	{
		if (!F.WaitFor(FTimespan::FromSeconds(30)))
		{
			AddError(TEXT("worker did not complete within 30s - possible deadlock"));
			return false;
		}
		TotalNulls += F.Get();
	}
	TestEqual(TEXT("no null snapshots returned under contention"), TotalNulls, 0);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FImgMediaTileVisibilityResolverNonBlockingTest,
	"System.Plugins.ImgMedia.TileVisibility.Resolver.GameThreadNonBlocking",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FImgMediaTileVisibilityResolverNonBlockingTest::RunTest(const FString&)
{
	using namespace ImgMediaTileVisibilityTest;

	FImgMediaTileVisibilityResolver Resolver;
	Resolver.SetSequenceInfo(MakeTestSequenceInfo());

	TSharedRef<FBlockingStubProvider> Blocker = MakeShared<FBlockingStubProvider>();
	Resolver.RegisterProvider(Blocker);

	TFuture<TSharedPtr<FMediaTileVisibilityRequest, ESPMode::ThreadSafe>> WorkerFuture =
		Async(EAsyncExecution::Thread, [&Resolver]()
		{
			return Resolver.GetSnapshot();
		});

	if (!Blocker->InsideEvent->Wait(5000))
	{
		AddError(TEXT("worker did not reach provider within 5s"));
		Blocker->ReleaseEvent->Trigger();
		return false;
	}

	// Worker is mid-GatherVisibleTiles. Pre-fix, all of these would block on
	// the resolver mutex until the worker exited - which never happens because
	// ReleaseEvent has not been triggered yet. Post-fix, none of them touch the
	// mutex while the worker is in provider code.
	const double StartTime = FPlatformTime::Seconds();
	TSharedRef<FStubProvider> Other = MakeShared<FStubProvider>(0, 1, 1, 4, 4);
	Resolver.RegisterProvider(Other);
	Resolver.InvalidateSnapshot();
	const int32 ProviderCount = Resolver.NumProviders();
	const double Elapsed = FPlatformTime::Seconds() - StartTime;

	TestTrue(TEXT("game thread completed resolver ops while worker in provider (< 0.5s)"), Elapsed < 0.5);
	TestEqual(TEXT("provider count reflects new registration"), ProviderCount, 2);

	Blocker->ReleaseEvent->Trigger();
	if (!WorkerFuture.WaitFor(FTimespan::FromSeconds(5)))
	{
		AddError(TEXT("worker did not complete after release"));
		return false;
	}
	TestTrue(TEXT("worker returned a valid snapshot"), WorkerFuture.Get().IsValid());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FImgMediaTileVisibilityResolverUnregisterDuringBuildTest,
	"System.Plugins.ImgMedia.TileVisibility.Resolver.UnregisterDuringBuild",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FImgMediaTileVisibilityResolverUnregisterDuringBuildTest::RunTest(const FString&)
{
	using namespace ImgMediaTileVisibilityTest;

	FImgMediaTileVisibilityResolver Resolver;
	Resolver.SetSequenceInfo(MakeTestSequenceInfo());

	TSharedRef<FBlockingStubProvider> Blocker = MakeShared<FBlockingStubProvider>();
	Resolver.RegisterProvider(Blocker);

	TFuture<TSharedPtr<FMediaTileVisibilityRequest, ESPMode::ThreadSafe>> WorkerFuture =
		Async(EAsyncExecution::Thread, [&Resolver]()
		{
			return Resolver.GetSnapshot();
		});

	if (!Blocker->InsideEvent->Wait(5000))
	{
		AddError(TEXT("worker did not reach provider within 5s"));
		Blocker->ReleaseEvent->Trigger();
		return false;
	}

	// Worker holds a strong pin from Phase 2 of GetSnapshot; unregistering must
	// not yank the rug from under the in-flight Gather call.
	Resolver.UnregisterProvider(Blocker);
	TestEqual(TEXT("resolver no longer references provider"), Resolver.NumProviders(), 0);

	Blocker->ReleaseEvent->Trigger();
	if (!WorkerFuture.WaitFor(FTimespan::FromSeconds(5)))
	{
		AddError(TEXT("worker did not complete after release"));
		return false;
	}
	TestTrue(TEXT("worker returned a valid snapshot"), WorkerFuture.Get().IsValid());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FImgMediaSphereProviderTransformTest,
	"System.Plugins.ImgMedia.TileVisibility.Resolver.SphereTransform",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FImgMediaSphereProviderTransformTest::RunTest(const FString&)
{
	using namespace UE::ImgMediaSphereVisibility::Tests;

	const FTransform Identity = FTransform::Identity;
	constexpr float Radius = 50.0f;
	constexpr float Eps = 1e-3f;

	auto NearlyEqualUV = [Eps](const FVector2f& A, const FVector2f& B)
	{
		return FMath::IsNearlyEqual(A.X, B.X, Eps) && FMath::IsNearlyEqual(A.Y, B.Y, Eps);
	};

	auto Roundtrip = [&](const FVector2f& MeshRange, const FVector2f& UV)
	{
		const FVector WS = TransformSphericalUVsToLocationWS(MeshRange, Identity, UV, Radius);
		return TransformDirectionWSToSphericalUVs(MeshRange, Identity, WS.GetSafeNormal());
	};

	// Sample interior UVs only - poles (V=0,1) are singularities in spherical coords
	// and longitudinal seam (U=0,1) wraps under Fmod.
	const TArray<float> Samples = {0.05f, 0.25f, 0.5f, 0.75f, 0.95f};

	// Full sphere: roundtrip uv -> position -> direction -> uv must be identity.
	{
		const FVector2f FullRange(360.f, 180.f);
		for (float U : Samples)
		{
			for (float V : Samples)
			{
				const FVector2f UV(U, V);
				const FVector2f Out = Roundtrip(FullRange, UV);
				TestTrue(*FString::Printf(TEXT("full-sphere roundtrip uv=(%.2f,%.2f)"), U, V),
					NearlyEqualUV(UV, Out));
			}
		}
	}

	// Partial sphere (180,90): roundtrip within mesh coverage must also be identity.
	{
		const FVector2f PartialRange(180.f, 90.f);
		for (float U : Samples)
		{
			for (float V : Samples)
			{
				const FVector2f UV(U, V);
				const FVector2f Out = Roundtrip(PartialRange, UV);
				TestTrue(*FString::Printf(TEXT("partial-sphere (180,90) roundtrip uv=(%.2f,%.2f)"), U, V),
					NearlyEqualUV(UV, Out));
			}
		}
	}

	// Partial sphere: directions outside coverage must produce UV outside [0,1].
	// This is the correct semantic for "camera looking past the partial mesh".
	{
		const FVector2f PartialRange(180.f, 90.f);

		// Mesh covers longitudes [0°,180°]. Direction at longitude 270° is outside.
		const FVector OutsideLong(0, -1, 0);
		const FVector2f UVLong = TransformDirectionWSToSphericalUVs(PartialRange, Identity, OutsideLong);
		TestTrue(TEXT("partial-sphere out-of-longitude produces UV.X > 1"), UVLong.X > 1.0f);

		// Mesh covers latitudes [45°,135°]. South pole (theta=π) is outside.
		const FVector SouthPole(0, 0, -1);
		const FVector2f UVPole = TransformDirectionWSToSphericalUVs(PartialRange, Identity, SouthPole);
		TestTrue(TEXT("partial-sphere south pole out-of-latitude produces UV.Y > 1"), UVPole.Y > 1.0f);
	}

	// Direction at the geometric center of the partial mesh inverts to UV ~ (0.5, 0.5).
	{
		const FVector2f PartialRange(180.f, 90.f);
		const FVector CenterDir(0, 1, 0);
		const FVector2f UV = TransformDirectionWSToSphericalUVs(PartialRange, Identity, CenterDir);
		TestTrue(TEXT("partial-sphere center direction inverts to UV ~ (0.5, 0.5)"),
			NearlyEqualUV(UV, FVector2f(0.5f, 0.5f)));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FImgMediaTileVisibilityResolverIdempotenceTest,
	"System.Plugins.ImgMedia.TileVisibility.Resolver.Idempotence",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FImgMediaTileVisibilityResolverIdempotenceTest::RunTest(const FString&)
{
	using namespace ImgMediaTileVisibilityTest;

	FImgMediaTileVisibilityResolver Resolver;
	Resolver.SetSequenceInfo(MakeTestSequenceInfo());

	TSharedRef<FStubProvider> P = MakeShared<FStubProvider>(0, 1, 1, 4, 4);
	Resolver.RegisterProvider(P);
	Resolver.RegisterProvider(P);
	Resolver.RegisterProvider(P);
	TestEqual(TEXT("repeated registration of the same provider is idempotent"),
		Resolver.NumProviders(), 1);

	// Unregister once removes the (single) entry.
	Resolver.UnregisterProvider(P);
	TestEqual(TEXT("single unregister removes the deduplicated entry"),
		Resolver.NumProviders(), 0);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FImgMediaTileVisibilityResolverClearProvidersTest,
	"System.Plugins.ImgMedia.TileVisibility.Resolver.ClearProviders",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FImgMediaTileVisibilityResolverClearProvidersTest::RunTest(const FString&)
{
	using namespace ImgMediaTileVisibilityTest;

	FImgMediaTileVisibilityResolver Resolver;
	Resolver.SetSequenceInfo(MakeTestSequenceInfo());

	TSharedRef<FStubProvider> P1 = MakeShared<FStubProvider>(0, 1, 1, 4, 4);
	TSharedRef<FStubProvider> P2 = MakeShared<FStubProvider>(0, 2, 2, 4, 4);
	Resolver.RegisterProvider(P1);
	Resolver.RegisterProvider(P2);

	if (const auto Snap = Resolver.GetSnapshot())
	{
		TestTrue(TEXT("snapshot has providers' contributions before clear"),
			Snap->VisibleTiles[0].IsVisible(1, 1) && Snap->VisibleTiles[0].IsVisible(2, 2));
	}

	Resolver.ClearProviders();
	TestEqual(TEXT("ClearProviders drops all providers"), Resolver.NumProviders(), 0);

	if (const auto Snap = Resolver.GetSnapshot())
	{
		TestEqual(TEXT("snapshot rebuilt empty after clear"), Snap->VisibleTiles.Num(), 0);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FImgMediaTileVisibilityResolverSetSequenceInfoInvalidatesTest,
	"System.Plugins.ImgMedia.TileVisibility.Resolver.SetSequenceInfoInvalidates",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FImgMediaTileVisibilityResolverSetSequenceInfoInvalidatesTest::RunTest(const FString&)
{
	using namespace ImgMediaTileVisibilityTest;

	FImgMediaTileVisibilityResolver Resolver;
	Resolver.SetSequenceInfo(MakeTestSequenceInfo());

	TSharedRef<FStubProvider> P = MakeShared<FStubProvider>(0, 1, 1, 4, 4);
	Resolver.RegisterProvider(P);

	const auto SnapBefore = Resolver.GetSnapshot();
	TestTrue(TEXT("initial snapshot is valid"), SnapBefore.IsValid());

	const auto SnapCached = Resolver.GetSnapshot();
	TestEqual(TEXT("cached call returns the same snapshot instance"),
		SnapCached.Get(), SnapBefore.Get());

	// Changing sequence info must invalidate the cache - next GetSnapshot rebuilds.
	FMediaSequenceInfo NewInfo = MakeTestSequenceInfo();
	NewInfo.Dim = FIntPoint(2048, 2048);
	Resolver.SetSequenceInfo(NewInfo);

	const auto SnapAfter = Resolver.GetSnapshot();
	TestTrue(TEXT("rebuilt snapshot is valid"), SnapAfter.IsValid());
	TestNotEqual(TEXT("SetSequenceInfo invalidates the cached snapshot"),
		SnapAfter.Get(), SnapBefore.Get());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FImgMediaQuadtreeMaxMipLevelTest,
	"System.Plugins.ImgMedia.TileVisibility.QuadtreeMaxMipLevel",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FImgMediaQuadtreeMaxMipLevelTest::RunTest(const FString&)
{
	using UE::ImgMediaProviders::ComputeQuadtreeMaxMipLevel;

	// Regression for UE-378546: NumMipLevels=14 with 64x32 tiles must clamp to the
	// tile-pyramid root (5), not the texture-mip count (13). At mip 6+ PartialTileNum
	// drops below 1 and tile centers walk off the plate by 64x plate width.
	TestEqual(TEXT("16384x8192 / 256, 14 mips: caps at FloorLog2(min(64,32))=5"),
		ComputeQuadtreeMaxMipLevel(14, FIntPoint(64, 32), /*bIsTiled=*/true), 5);

	// Power-of-two square tile grid: cap matches the natural mip count.
	TestEqual(TEXT("16x16 tiles, 5 mips: cap = 4"),
		ComputeQuadtreeMaxMipLevel(5, FIntPoint(16, 16), true), 4);

	// NumMipLevels is the binding cap when smaller than the tile pyramid.
	TestEqual(TEXT("16x16 tiles, 3 mips: NumMipLevels-1 wins"),
		ComputeQuadtreeMaxMipLevel(3, FIntPoint(16, 16), true), 2);

	// Asymmetric grid: log2(min) — Y axis bottoms out first.
	TestEqual(TEXT("8x2 tiles: cap = log2(2) = 1"),
		ComputeQuadtreeMaxMipLevel(10, FIntPoint(8, 2), true), 1);

	// Single tile per axis: no quadtree.
	TestEqual(TEXT("1x1 tiles: cap = 0"),
		ComputeQuadtreeMaxMipLevel(5, FIntPoint(1, 1), true), 0);

	// Non-tiled sequences keep current behavior (NumMipLevels-1, no tile cap).
	TestEqual(TEXT("non-tiled, 5 mips: cap = 4"),
		ComputeQuadtreeMaxMipLevel(5, FIntPoint(1, 1), /*bIsTiled=*/false), 4);
	TestEqual(TEXT("non-tiled, 1 mip: cap = 0"),
		ComputeQuadtreeMaxMipLevel(1, FIntPoint(1, 1), false), 0);

	// Defensive edges: zero/negative inputs must not crash or return negatives.
	TestEqual(TEXT("0 mips floors at 0"),
		ComputeQuadtreeMaxMipLevel(0, FIntPoint(4, 4), true), 0);
	TestEqual(TEXT("zero tile counts treated as 1 to avoid FloorLog2(0)"),
		ComputeQuadtreeMaxMipLevel(5, FIntPoint(0, 0), true), 0);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FImgMediaEmitVisibilityAboveQuadtreeCapTest,
	"System.Plugins.ImgMedia.TileVisibility.EmitVisibilityAboveQuadtreeCap",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FImgMediaEmitVisibilityAboveQuadtreeCapTest::RunTest(const FString&)
{
	using UE::ImgMediaProviders::EmitVisibilityAboveQuadtreeCap;

	const FIntPoint Dim(16384, 8192);
	const FIntPoint TileSize(256, 256);
	constexpr int32 MaxLevel = 5;

	// Far-camera: calc mip = 13, MipLevelRange = [13, 13]. Emits a single-tile entry at mip 13.
	{
		FMediaTileVisibilityRequest Req;
		EmitVisibilityAboveQuadtreeCap(Dim, TileSize, MaxLevel, FIntVector2(13, 13), Req);
		TestEqual(TEXT("far view emits one mip entry"), Req.VisibleTiles.Num(), 1);
		TestTrue(TEXT("mip 13 emitted"), Req.VisibleTiles.Contains(13));
		TestEqual(TEXT("mip 13 single-tile dimensions"),
			Req.VisibleTiles[13].GetDimensions(), FIntPoint(1, 1));
		TestTrue(TEXT("mip 13 (0,0) marked visible"),
			Req.VisibleTiles[13].IsVisible(0, 0));
	}

	// Mid-range range straddling the cap: emit only mips above the cap (6..7), the in-range mips
	// (0..5) are the quadtree's responsibility and must not be touched here.
	{
		FMediaTileVisibilityRequest Req;
		EmitVisibilityAboveQuadtreeCap(Dim, TileSize, MaxLevel, FIntVector2(2, 7), Req);
		TestEqual(TEXT("range [2,7] emits mips 6 and 7"), Req.VisibleTiles.Num(), 2);
		TestTrue(TEXT("mip 6 emitted"), Req.VisibleTiles.Contains(6));
		TestTrue(TEXT("mip 7 emitted"), Req.VisibleTiles.Contains(7));
		TestFalse(TEXT("mip 5 not emitted (quadtree's domain)"),
			Req.VisibleTiles.Contains(5));
	}

	// Range entirely at or below cap: emit nothing.
	{
		FMediaTileVisibilityRequest Req;
		EmitVisibilityAboveQuadtreeCap(Dim, TileSize, MaxLevel, FIntVector2(0, 5), Req);
		TestEqual(TEXT("range [0,5] emits nothing"), Req.VisibleTiles.Num(), 0);
	}

	// Idempotence under repeated calls (multiple root tiles emit the same high mips).
	{
		FMediaTileVisibilityRequest Req;
		EmitVisibilityAboveQuadtreeCap(Dim, TileSize, MaxLevel, FIntVector2(13, 13), Req);
		EmitVisibilityAboveQuadtreeCap(Dim, TileSize, MaxLevel, FIntVector2(13, 13), Req);
		TestEqual(TEXT("repeated emit stays at one mip entry"), Req.VisibleTiles.Num(), 1);
		TestTrue(TEXT("mip 13 still (0,0) visible"),
			Req.VisibleTiles[13].IsVisible(0, 0));
	}

	// Asymmetric grid: 16x1 tiles, cap at log2(1)=0, MipLevelRange [3,3]. At mip 3 the X axis
	// still has ceil(16/8/1)=2 tiles. The all-visible selection must reflect that, not (1,1).
	{
		FMediaTileVisibilityRequest Req;
		EmitVisibilityAboveQuadtreeCap(FIntPoint(4096, 256), FIntPoint(256, 256),
			/*MaxLevel=*/0, FIntVector2(3, 3), Req);
		TestTrue(TEXT("asymmetric mip 3 emitted"), Req.VisibleTiles.Contains(3));
		TestEqual(TEXT("asymmetric dims reflect remaining X-axis tiles"),
			Req.VisibleTiles[3].GetDimensions(), FIntPoint(2, 1));
		TestTrue(TEXT("asymmetric mip 3 tile (0,0) visible"),
			Req.VisibleTiles[3].IsVisible(0, 0));
		TestTrue(TEXT("asymmetric mip 3 tile (1,0) visible"),
			Req.VisibleTiles[3].IsVisible(1, 0));
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMediaTextureTrackerCleanupTest,
	"System.Plugins.ImgMedia.TileVisibility.TextureTrackerCleanup",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMediaTextureTrackerCleanupTest::RunTest(const FString&)
{
	// Strong-ref keeps the texture alive across the test; releasing on scope exit
	// lets GC collect it, leaving the singleton tracker in a clean state for other tests.
	TStrongObjectPtr<UMediaTexture> Texture(NewObject<UMediaTexture>());

	FMediaTextureTracker& Tracker = FMediaTextureTracker::Get();
	const int32 BaselineTextureCount = Tracker.GetTextures().Num();

	auto ContainsTexture = [&Texture](const TArray<TWeakObjectPtr<UMediaTexture>>& Textures)
	{
		return Textures.ContainsByPredicate(
			[&Texture](const TWeakObjectPtr<UMediaTexture>& W) { return W.Get() == Texture.Get(); });
	};

	FMediaTextureTrackerObjectPtr Info = MakeShared<FMediaTextureTrackerObject, ESPMode::ThreadSafe>();

	Tracker.RegisterTexture(Info, Texture.Get());
	TestTrue(TEXT("texture appears in GetTextures after register"), ContainsTexture(Tracker.GetTextures()));
	TestNotNull(TEXT("GetObjects returns the registration after register"), Tracker.GetObjects(Texture.Get()));

	Tracker.UnregisterTexture(Info, Texture.Get());
	TestFalse(TEXT("texture removed from GetTextures after last consumer unregisters"),
		ContainsTexture(Tracker.GetTextures()));
	TestTrue(TEXT("GetObjects returns nullptr after last consumer unregisters"),
		Tracker.GetObjects(Texture.Get()) == nullptr);
	TestEqual(TEXT("baseline texture count restored - no entry leaked"),
		Tracker.GetTextures().Num(), BaselineTextureCount);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMediaTextureTrackerNullUnregisterCleanupTest,
	"System.Plugins.ImgMedia.TileVisibility.TextureTrackerNullUnregisterCleanup",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMediaTextureTrackerNullUnregisterCleanupTest::RunTest(const FString&)
{
	TStrongObjectPtr<UMediaTexture> Texture(NewObject<UMediaTexture>());
	FMediaTextureTracker& Tracker = FMediaTextureTracker::Get();
	const int32 BaselineTextureCount = Tracker.GetTextures().Num();

	FMediaTextureTrackerObjectPtr Info = MakeShared<FMediaTextureTrackerObject, ESPMode::ThreadSafe>();
	Tracker.RegisterTexture(Info, Texture.Get());

	// Slow-path scan: caller passes nullptr to simulate "lost the texture pointer".
	// Cleanup must still happen even though the fast lookup path is skipped.
	Tracker.UnregisterTexture(Info, nullptr);

	TestTrue(TEXT("GetObjects returns nullptr after null-unregister"),
		Tracker.GetObjects(Texture.Get()) == nullptr);
	TestEqual(TEXT("baseline texture count restored after null-unregister"),
		Tracker.GetTextures().Num(), BaselineTextureCount);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMediaTextureTrackerBroadcastInfoTest,
	"System.Plugins.ImgMedia.TileVisibility.TextureTrackerBroadcastInfo",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMediaTextureTrackerBroadcastInfoTest::RunTest(const FString&)
{
	TStrongObjectPtr<UMediaTexture> Texture(NewObject<UMediaTexture>());
	FMediaTextureTracker& Tracker = FMediaTextureTracker::Get();
	FMediaTextureTrackerObjectPtr Info = MakeShared<FMediaTextureTrackerObject, ESPMode::ThreadSafe>();

	UMediaTexture* CapturedRegisterTexture = nullptr;
	FMediaTextureTrackerObjectPtr CapturedRegisterInfo;
	UMediaTexture* CapturedUnregisterTexture = nullptr;
	FMediaTextureTrackerObjectPtr CapturedUnregisterInfo;

	const FDelegateHandle RegHandle = Tracker.OnObjectRegistered().AddLambda(
		[&](TObjectPtr<UMediaTexture> T, const FMediaTextureTrackerObjectPtr& I)
		{
			CapturedRegisterTexture = T;
			CapturedRegisterInfo = I;
		});
	const FDelegateHandle UnregHandle = Tracker.OnObjectUnregistered().AddLambda(
		[&](TObjectPtr<UMediaTexture> T, const FMediaTextureTrackerObjectPtr& I)
		{
			CapturedUnregisterTexture = T;
			CapturedUnregisterInfo = I;
		});

	Tracker.RegisterTexture(Info, Texture.Get());
	TestEqual(TEXT("OnObjectRegistered receives the correct texture"),
		CapturedRegisterTexture, Texture.Get());
	TestTrue(TEXT("OnObjectRegistered receives the correct info"),
		CapturedRegisterInfo == Info);

	Tracker.UnregisterTexture(Info, Texture.Get());
	TestEqual(TEXT("OnObjectUnregistered receives the correct texture"),
		CapturedUnregisterTexture, Texture.Get());
	TestTrue(TEXT("OnObjectUnregistered receives the correct info"),
		CapturedUnregisterInfo == Info);

	Tracker.OnObjectRegistered().Remove(RegHandle);
	Tracker.OnObjectUnregistered().Remove(UnregHandle);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FImgMediaTileVisibilityResolverBroadcastRegisterRaceTest,
	"System.Plugins.ImgMedia.TileVisibility.Resolver.BroadcastRegisterRace",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FImgMediaTileVisibilityResolverBroadcastRegisterRaceTest::RunTest(const FString&)
{
	using namespace ImgMediaTileVisibilityTest;

	FImgMediaTileVisibilityResolver Resolver;
	Resolver.SetSequenceInfo(MakeTestSequenceInfo());

	TSharedRef<FStubProvider> P = MakeShared<FStubProvider>(0, 1, 1, 4, 4);
	Resolver.RegisterProvider(P);

	// Lock-order regression: BroadcastOnInputsChanged takes the provider's OnInputsChanged
	// mutex and invokes the resolver's InvalidateSnapshot handler, which then takes the
	// resolver's Mutex. Register/Unregister take the resolver's Mutex first, then the
	// provider's OnInputsChanged mutex via Add/RemoveOnInputsChangedHandler. If both
	// threads can hold their first lock while contending for the second, they deadlock.
	// The provider snapshot-and-dispatches outside its mutex so handler invocation no
	// longer holds the provider lock; cycling and broadcasting must therefore complete.
	constexpr int32 Iterations = 2000;
	constexpr double TimeoutSec = 10.0;

	TFuture<void> Broadcaster = Async(EAsyncExecution::Thread, [&P]()
	{
		for (int32 i = 0; i < Iterations; ++i)
		{
			P->BroadcastOnInputsChanged();
		}
	});

	TFuture<void> Cycler = Async(EAsyncExecution::Thread, [&Resolver, &P]()
	{
		for (int32 i = 0; i < Iterations; ++i)
		{
			Resolver.UnregisterProvider(P);
			Resolver.RegisterProvider(P);
		}
	});

	if (!Broadcaster.WaitFor(FTimespan::FromSeconds(TimeoutSec)))
	{
		AddError(TEXT("broadcaster did not complete within 10s - lock-order regression between provider mutex and resolver mutex"));
		return false;
	}
	if (!Cycler.WaitFor(FTimespan::FromSeconds(TimeoutSec)))
	{
		AddError(TEXT("cycler did not complete within 10s - lock-order regression between resolver mutex and provider mutex"));
		return false;
	}

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
