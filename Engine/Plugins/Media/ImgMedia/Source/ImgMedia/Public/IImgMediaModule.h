// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Templates/SharedPointer.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define UE_API IMGMEDIA_API

class FImgMediaGlobalCache;
class FImgMediaPlayer;
class FImgMediaSceneViewExtension;
class IMediaEventSink;
class IMediaPlayer;

CSV_DECLARE_CATEGORY_MODULE_EXTERN(IMGMEDIA_API, ImgMedia);

/** Callback when a player gets created. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnImgMediaPlayerCreated, const TSharedPtr<FImgMediaPlayer>&);

/**
 * Interface for the ImgMedia module.
 */
class IImgMediaModule
	: public IModuleInterface
{
public:

	/**
	* Singleton-like access to this module's interface.  This is just for convenience!
	* Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	*
	* @return Returns singleton instance, loading the module on demand if needed
	*/
	static inline IImgMediaModule& Get()
	{
		static IImgMediaModule* ImgMediaModulePtr = nullptr;
		if (!ImgMediaModulePtr)
		{
			ImgMediaModulePtr = FModuleManager::LoadModulePtr<IImgMediaModule>("ImgMedia");
		}

		return *ImgMediaModulePtr;
	}

	/**
	 * Creates a media player for image sequences.
	 *
	 * @param EventHandler The object that will receive the player's events.
	 * @return A new media player, or nullptr if a player couldn't be created.
	 */
	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventHandler) = 0;

	/** Get scene view extension caching view information (for mip/tile calculations). */
	virtual const TSharedPtr<FImgMediaSceneViewExtension, ESPMode::ThreadSafe>& GetSceneViewExtension() const = 0;
public:

	/** Virtual destructor. */
	virtual ~IImgMediaModule() = default;

	/** Add to this callback to get called whenever a player is created. */
	FOnImgMediaPlayerCreated OnImgMediaPlayerCreated;

	/**
	 * Call this to get the global cache.
	 *
	 * @return Global cache.
	 */
	static FImgMediaGlobalCache* GetGlobalCache() { return GlobalCache.Get(); }

	/** Name of attribute in the Exr file that marks it as our custom format. */
	static UE_API FLazyName CustomFormatAttributeName;
	/** Name of attribute in the Exr file for the tile width for our custom format. */
	static UE_API FLazyName CustomFormatTileWidthAttributeName;
	/** Name of attribute in the Exr file for the tile height for our custom format. */
	static UE_API FLazyName CustomFormatTileHeightAttributeName;
	/** Name of attribute in the Exr file for the tile border size for our custom format. */
	static UE_API FLazyName CustomFormatTileBorderAttributeName;

protected:

	/** Holds the global cache. */
	static UE_API TSharedPtr<FImgMediaGlobalCache, ESPMode::ThreadSafe> GlobalCache;
};

#undef UE_API
