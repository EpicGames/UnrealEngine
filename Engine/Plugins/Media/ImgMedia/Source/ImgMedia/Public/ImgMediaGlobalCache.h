// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Delegates/IDelegateInstance.h"
#include "Templates/SharedPointer.h"

#define UE_API IMGMEDIA_API

struct FFileChangeData; 
struct FImgMediaFrame;
class UImgMediaSettings;

/**
 * A global cache for all ImgMedia players.
 *
 * Uses Least Recently Used (LRU).
 */
class FImgMediaGlobalCache : public TSharedFromThis<FImgMediaGlobalCache, ESPMode::ThreadSafe>
{
public:
	/**
	 * Constructor.
	 */
	UE_API FImgMediaGlobalCache();

	/**
	 * Desctructor.
	  */
	UE_API ~FImgMediaGlobalCache();

	/**
	 * Initialize the cache.
	 */
	UE_API void Initialize();

	/**
	 * Shut down the cache.
	 */
	UE_API void Shutdown();

	/**
	 * Adds a frame to the cache.
	 *
	 * @param	FileName		Filename of frame to add.
	 * @param	Sequence		Indentifying name of this sequence.
	 * @param	Index			Index of frame to add.
	 * @param	Frame			Actual frame to add.
	 * @param	bAdjustSizeForMipMaps	Whether this frame needs size adjustments for mip maps in separate folders.
	 */
	UE_API void AddFrame(const FString& FileName, const FName& Sequence, int32 Index, const TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>& Frame, bool bAdjustSizeForMipMaps = false);

	/**
	 * Check whether the entry with the specified sequence and index is present.
	 *
	 * @param	Sequence		Indentifying name of this sequence.
	 * @param	Index			Index of frame to touch.
	 */
	UE_API bool Contains(const FName& Sequence, int32 Index);

	/**
	 * Find the entry with the specified sequence and index and mark it as the most recently used.
	 *
	 * @param	Sequence		Indentifying name of this sequence.
	 * @param	Index			Index of frame to touch.
	 */
	UE_API TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>* FindAndTouch(const FName& Sequence, int32 Index);

	/**
	 * Find the indices of all cached entries of a particular sequence.
	 *
	 * @param	Sequence		Indentifying name of this sequence.
	 * @param OutIndices Will contain the collection of indices.
	 */
	UE_API void GetIndices(const FName& Sequence, TArray<int32>& OutIndices) const;

	/**
	 * Empties the cache.
	 */
	UE_API void EmptyCache();

	/**
	 * Get the current size (in bytess).
	 */
	SIZE_T GetCurrentSize() const { return CurrentSize; }

	/**
	 * Get the maximum size (in bytes).
	 */
	SIZE_T GetMaxSize() const { return MaxSize; }

private:

	/** An entry in the cache. */
	struct FImgMediaGlobalCacheEntry
	{
		/** File name of this frame. */
		FString FileName;
		/** Size of this frame in bytes. */
		SIZE_T FrameSize;
		/** Frame index. */
		int32 Index;
		/** Actual frame. */
		TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe> Frame;

		/** Previous entry in the cache. */
		FImgMediaGlobalCacheEntry* LessRecent;
		/** Previous entry in the cache that is part of the same sequence. */
		FImgMediaGlobalCacheEntry* LessRecentSequence;
		/** Next entry in the cache. */
		FImgMediaGlobalCacheEntry* MoreRecent;
		/** Next entry in the cache that is part of the same sequence. */
		FImgMediaGlobalCacheEntry* MoreRecentSequence;

		/** Constructor. */
		FImgMediaGlobalCacheEntry(const FString& InFileName, SIZE_T InFrameSize, int32 InIndex, const TSharedPtr<FImgMediaFrame, ESPMode::ThreadSafe>& InFrame)
			: FileName(InFileName)
			, FrameSize(InFrameSize)
			, Index(InIndex)
			, Frame(InFrame)
			, LessRecent(nullptr)
			, LessRecentSequence(nullptr)
			, MoreRecent(nullptr)
			, MoreRecentSequence(nullptr)
		{
		}
	};

	/** Entry that was used first. */
	FImgMediaGlobalCacheEntry* LeastRecent;
	/** Entry that was used last. */
	FImgMediaGlobalCacheEntry* MostRecent;

	/** Maps a sequence name to the most recent cache entry of that sequence. */
	TMap<FName, FImgMediaGlobalCacheEntry*> MapSequenceToMostRecentEntry;
	/** Maps a cache entry that is the least recent entry to the name of its sequence. */
	TMap<FImgMediaGlobalCacheEntry*, FName> MapLeastRecentToSequence;
	/** Maps a sequence name and frame index to an entry in the cache. */
	TMap<TPair<FName, int32>, FImgMediaGlobalCacheEntry*> MapFrameToEntry;

	/** Current size of the cache in bytes. */
	SIZE_T CurrentSize;
	/** Maximum size of the cache in bytes. */
	SIZE_T MaxSize;

	/** Critical section for synchronizing access to Frames. */
	mutable FCriticalSection CriticalSection;

#if WITH_EDITORONLY_DATA

	/** Handle to registered UpdateSettings delegate. */
	FDelegateHandle UpdateSettingsDelegateHandle;

#endif

#if WITH_EDITOR

	/** Maps a sequence directory to a directory watcher. */
	TMap<FString, FDelegateHandle> MapSequenceToWatcher;
	/** Maps a file name to the index of that file in the sequence. */
	TMap<FString, int32> MapFileToIndex;

#endif // WITH_EDITOR
	
	/**
	 * Empties the cache until the current size + Extra <= the max size of the cache.
	 */
	UE_API void EnforceMaxSize(SIZE_T Extra);

	/**
	 * Removes an entry from the cache and deletes the entry.
	 *
	 * @param	Sequence		Indentifying name of this sequence.
	 * @param	Entry			Entry to remove.
	 */
	UE_API void Remove(const FName& Sequence, FImgMediaGlobalCacheEntry& Entry);

	/**
	 * Inserts an entry into the cache as the most recent entry.
	 * Assumes the entry is not already in the cache.
	 *
	 * @param	Sequence		Indentifying name of this sequence.
	 * @param	Entry			Entry to mark as recent.
	 */
	UE_API void MarkAsRecent(const FName& Sequence, FImgMediaGlobalCacheEntry& Entry);

	/**
	 * Removes an entry from the cache but does not delete the entry.
	 *
	 * @param	Sequence		Indentifying name of this sequence.
	 * @param	Entry			Entry to remove.
	 */
	UE_API void Unlink(const FName& Sequence, FImgMediaGlobalCacheEntry& Entry);

	/**
	 * Empties the cache.
	 * 
	 * Assumes CriticalSection is already locked.
	 */
	UE_API void Empty();

	/** 
	 * Updates our settings from ImgMediaAsettings. 
	 */
	UE_API void UpdateSettings(const UImgMediaSettings* Settings);

#if WITH_EDITOR

	/**
	 * Called when one of our source files changes.
	 */
	UE_API void OnDirectoryChanged(const TArray<FFileChangeData>& InFileChanges);

#endif // WITH_EDITOR
};

#undef UE_API
