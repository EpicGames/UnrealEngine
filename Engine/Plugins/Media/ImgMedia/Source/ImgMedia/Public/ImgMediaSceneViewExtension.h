// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MediaTextureTracker.h"
#include "Containers/Set.h"
#include "HAL/CriticalSection.h"
#include "SceneTypes.h"
#include "SceneViewExtension.h"

/**
 * Holds info on a camera which we can use for mipmap calculations.
 */
struct FImgMediaViewInfo
{
	/** Position of camera. */
	FVector Location;
	/** View direction of the camera. */
	FVector ViewDirection;
	/** View-projection matrix of the camera. */
	FMatrix ViewProjectionMatrix;
	/** View-projection matrix of the camera, optionally scaled for overscan frustum calculations. */
	FMatrix OverscanViewProjectionMatrix;
	/** Active viewport size. */
	FIntRect ViewportRect;
	/** View mip bias. */
	float MaterialTextureMipBias;
	/** Hidden or show-only mode for primitive components. */
	bool bPrimitiveHiddenMode;
	/** Hidden or show-only primitive components. */
	TSet<FPrimitiveComponentId> PrimitiveComponentIds;
};

/**
 * Scene view extension used to cache view information (primarily for visible mip/tile calculations).
 */
class FImgMediaSceneViewExtension final : public FSceneViewExtensionBase
{
public:
	explicit FImgMediaSceneViewExtension(const FAutoRegister& AutoReg);
	virtual ~FImgMediaSceneViewExtension() override;

	/**
	 * Select the appropriate set(s) of cached view infos based on the target resolution mask.
	 * When DisplayResolution is requested but the display-resolution cache is empty (which is
	 * the common case where render resolution equals display resolution), the render-resolution
	 * cache is returned instead. Callers that need to distinguish the two cases must check the
	 * cache state externally - this method does not signal which set was actually returned.
	 * @param InTargetViewResolutionMask Mask to select the set of view infos.
	 * @return Combined ViewInfos.
	 */
	TArray<FImgMediaViewInfo> GetViewInfos(EMediaTextureTargetViewResolution InTargetViewResolutionMask) const;

	//~ Begin ISceneViewExtension
	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;
	virtual int32 GetPriority() const override;
	//~ End ISceneViewExtension

private:
	/** Cache camera view information for the current frame. Caller must hold ViewInfosMutex. */
	void CacheViewInfo_NoLock(FSceneViewFamily& InViewFamily, const FSceneView& View);

	/** Reset the view info cache. Caller must hold ViewInfosMutex. */
	void ResetViewInfoCache_NoLock();

	/**
	 * Protects CachedViewInfos and DisplayResolutionCachedViewInfos.
	 * BeginRenderViewFamily holds it across the reset+populate sequence on the game thread;
	 * GetViewInfos holds it across the copy on worker threads. This closes the race where a
	 * worker iterating CachedViewInfos could observe a TArray::Add reallocation mid-iteration
	 * (UAF) or a torn Reset->Add transition.
	 */
	mutable FCriticalSection ViewInfosMutex;

	/** 
	 * Array of info on each camera used for mipmap calculations.
	 * Updated on the game thread by BeginRenderViewFamily.
	 * Guarded by ViewInfosMutex. 
	 */
	TArray<FImgMediaViewInfo> CachedViewInfos;

	/**
	 * Array of cached camera information at display resolution for compositing.
	 * Will remain empty if the render resolution matches the display resolution.
	 * Updated on the game thread by BeginRenderViewFamily.
	 * Guarded by ViewInfosMutex.
	 */
	TArray<FImgMediaViewInfo> DisplayResolutionCachedViewInfos;

	/**
	 * GFrameNumber when CachedViewInfos was last cleared+populated.
	 */
	uint64 LastFrameNumber = ~0ull;
};
