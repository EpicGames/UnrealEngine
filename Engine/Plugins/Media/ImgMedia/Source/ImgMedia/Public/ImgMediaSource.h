// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Engine/EngineTypes.h"
#include "Containers/UnrealString.h"
#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "BaseMediaSource.h"
#include "BaseMediaSourceColorSettings.h"

#include "ImgMediaSource.generated.h"

#define UE_API IMGMEDIA_API

class AActor;

/** This provides customized editing of SequencePath. */
USTRUCT()
struct FImgMediaSourceCustomizationSequenceProxy
{
	GENERATED_BODY()
};


/**
 * Media source for EXR image sequences.
 *
 * Image sequence media sources point to a directory that contains a series of
 * image files in which each image represents a single frame of the sequence.
 * BMP, EXR, PNG and JPG images are currently supported. EXR image sequences
 * are optimized for performance. The first frame of an image sequence is used
 * to determine the image dimensions (all formats) and frame rate (EXR only).
 *
 * The image sequence directory may contain sub-directories, which are called
 * 'proxies'. Proxies can be used to provide alternative media for playback
 * during development and testing of a game. One common scenario is the use
 * of low resolution versions of image sequence media on computers that are
 * too slow or don't have enough storage to play the original high-res media.
 */
UCLASS(MinimalAPI, BlueprintType, hidecategories=(Overrides, Playback))
class UImgMediaSource
	: public UBaseMediaSource
{
	GENERATED_BODY()

public:

	/** Default constructor. */
	UE_API UImgMediaSource();

public:

	/** If true, then relative Sequence Paths are relative to the project root directory. If false, then relative to the Content directory. */
	UPROPERTY()
	bool IsPathRelativeToProjectRoot_DEPRECATED;

	/** Overrides the default frame rate stored in the image files (0/0 = do not override). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Sequence, AdvancedDisplay)
	FFrameRate FrameRateOverride;

	/** Name of the proxy directory to use. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Sequence, AdvancedDisplay)
	FString ProxyOverride;

	/** (EXR only) Name of the layer to read. Empty reads the default (un-prefixed) channels. Supports wildcards (* and ?). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Sequence, AdvancedDisplay, meta=(DisplayName="Layer Name (EXR)"))
	FString LayerName;

	/** If true, then any gaps in the sequence will be filled with blank frames. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Sequence)
	bool bFillGapsInSequence;

#if WITH_EDITORONLY_DATA

	/** This is only used so we can customize editing of SequencePath. */
	UPROPERTY(EditAnywhere, Transient, Category = Sequence)
	FImgMediaSourceCustomizationSequenceProxy SequenceProxy;

#endif // WITH_EDITORONLY_DATA

	/** Specification of a timecode associated with the start of the sequence. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Timecode")
	FTimecode StartTimecode;

	/** Manual definition of media source color space & encoding. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Sequence", meta = (DisplayAfter = "SequencePath", ShowOnlyInnerProperties))
	FMediaSourceColorSettings SourceColorSettings;

public:

	/**
	 * Get the names of available proxy directories.
	 *
	 * @param OutProxies Will contain the names of available proxy directories, if any.
	 * @see GetSequencePath
	 */
	UFUNCTION(BlueprintCallable, Category="ImgMedia|ImgMediaSource")
	UE_API void GetProxies(TArray<FString>& OutProxies) const;

	/**
	 * Get the path to the image sequence directory to be played. Supported tokens will be expanded.
	 *
	 * @return The file path.
	 * @see SetSequencePath
	 */
	UFUNCTION(BlueprintCallable, Category = "ImgMedia|ImgMediaSource")
	UE_API const FString GetSequencePath() const;

	/**
	 * Set the path to the image sequence directory this source represents.
	 *
	 * @param Path The path to an image file in the desired directory.
	 * @see GetSequencePath
	 */
	UFUNCTION(BlueprintCallable, Category="ImgMedia|ImgMediaSource")
	UE_API void SetSequencePath(const FString& Path);

	/**
	 * Set the path to the image sequence directory this source represents.
	 *
	 * @param Path The path to the desired image sequence directory. May contain supported tokens.
	 * @see GetSequencePath
	 */
	UFUNCTION(BlueprintCallable, Category = "ImgMedia|ImgMediaSource")
	UE_API void SetTokenizedSequencePath(const FString& Path);

	/**
	 * Deprecated. Tile-visibility registration has moved to FMediaTextureTracker; consumers
	 * populate FMediaTextureTrackerObject::TileVisibilityProvider (or the legacy enum fields)
	 * directly instead of registering actors against the source. This function is now a no-op.
	 *
	 * @param InActor Ignored.
	 */
	UFUNCTION(BlueprintCallable, Category = "ImgMedia|ImgMediaSource", meta = (DeprecatedFunction, DeprecationMessage = "AddTargetObject is no longer needed. Tile-visibility consumers register through FMediaTextureTracker."))
	UE_DEPRECATED(5.8, "AddTargetObject is no longer needed. Tile-visibility consumers register through FMediaTextureTracker.")
	UE_API void AddTargetObject(AActor* InActor);

	/**
	 * Deprecated. Pair to AddTargetObject; now a no-op for the same reason.
	 *
	 * @param InActor Ignored.
	 */
	UFUNCTION(BlueprintCallable, Category = "ImgMedia|ImgMediaSource", meta = (DeprecatedFunction, DeprecationMessage = "RemoveTargetObject is no longer needed. Tile-visibility consumers register through FMediaTextureTracker."))
	UE_DEPRECATED(5.8, "RemoveTargetObject is no longer needed. Tile-visibility consumers register through FMediaTextureTracker.")
	UE_API void RemoveTargetObject(AActor* InActor);


public:

	//~ IMediaOptions interface

	UE_API virtual bool GetMediaOption(const FName& Key, bool DefaultValue) const override;
	UE_API virtual int64 GetMediaOption(const FName& Key, int64 DefaultValue) const override;
	UE_API virtual FString GetMediaOption(const FName& Key, const FString& DefaultValue) const override;
	UE_API virtual TSharedPtr<FDataContainer, ESPMode::ThreadSafe> GetMediaOption(const FName& Key, const TSharedPtr<FDataContainer, ESPMode::ThreadSafe>& DefaultValue) const override;
	UE_API virtual bool HasMediaOption(const FName& Key) const override;

public:

	//~ UMediaSource interface

	UE_API virtual FString GetUrl() const override;
	UE_API virtual bool Validate() const override;

#if WITH_EDITOR
	UE_API virtual void GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const override;
	
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

public:

	//~ UObject interface

	UE_API virtual void Serialize(FArchive& Ar) override;

public:

	/* Returns the path after replacing the supported tokens */
	static UE_API FString ExpandSequencePathTokens(const FString& InPath);

	/* Returns a sanitized sequence path, but without expanding the tokens tokens */
	static UE_API FString SanitizeTokenizedSequencePath(const FString& InPath);

	/* Returns true if InPath is under InBasePath. If so, fills out OutRelativePath with the relative path between them */
	static UE_API bool IsPathUnderBasePath(const FString& InPath, const FString& InBasePath, FString& OutRelativePath);

public:

	/** Get the full path to the image sequence. */
	UE_API FString GetFullPath() const;

protected:

	/** The directory that contains the image sequence files.
	 *  
	 * - Relative paths will be with respect to the current Project directory. 
	 * - You may use {engine_dir} or {project_dir} tokens.
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Sequence, meta = (EditCondition = "false", EditConditionHides))
	FDirectoryPath SequencePath;

	/** Native source color settings. */
	TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> NativeSourceColorSettings;
};

#undef UE_API
