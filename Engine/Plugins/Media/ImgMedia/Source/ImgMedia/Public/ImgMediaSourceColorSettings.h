// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

UE_DEPRECATED_HEADER(5.8, "Use BaseMediaSourceColorSettings.h from the MediaAssets module instead of ImgMediaSourceColorSettings.h.")

#include "BaseMediaSourceColorSettings.h"
