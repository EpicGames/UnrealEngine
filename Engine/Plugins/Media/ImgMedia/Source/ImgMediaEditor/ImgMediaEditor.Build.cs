// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ImgMediaEditor : ModuleRules
	{
		public ImgMediaEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AssetDefinition",
					"ContentBrowser",
					"Core",
					"CoreUObject",
					"DesktopWidgets",				
					"EditorFramework",
					"ImageCore",
					"ImageWrapper",
					"ImgMedia",
					"InputCore",
					"MediaAssets",
					"MediaPlayerEditor",
					"MediaUtils",
					"RenderCore",
					"Slate",
					"SlateCore",
					"TmvMedia",
					"ToolMenus",
					"UnrealEd",
					"WorkspaceMenuStructure",
				});

			PrivateIncludePaths.AddRange(
				new string[] {
					//"ImgMedia/Private",
				});

			// Are we using the engine?
			if (Target.bCompileAgainstEngine)
			{
				PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Engine",
				});
			}

			bool bLinuxEnabled = Target.Platform == UnrealTargetPlatform.Linux && Target.Architecture == UnrealArch.X64;

			// Add EXR support.
			if ((Target.Platform == UnrealTargetPlatform.Mac) ||
				Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) ||
				bLinuxEnabled)
			{
				PrivateDependencyModuleNames.Add("OpenExrWrapper");
				PublicDefinitions.Add("IMGMEDIAEDITOR_EXR_SUPPORTED_PLATFORM=1");
			}
			else
			{
				PublicDefinitions.Add("IMGMEDIAEDITOR_EXR_SUPPORTED_PLATFORM=0");
			}
		}
	}
}
