// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ImgMediaSource.h"
#include "AssetDefinition/AssetDefinition_MediaSource.h"
#include "AssetDefinition_ImgMediaSource.generated.h"

UCLASS()
class UAssetDefinition_ImgMediaSource : public UAssetDefinition_MediaSource
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_ImgMediaSource", "Img Media Source"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UImgMediaSource::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Media, NSLOCTEXT("AssetDefinition", "AssetDefinition_ImgMediaSourceSubMenu", "Media Sources + Outputs"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
