// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImgMediaFilePathCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "Misc/ConfigCacheIni.h"
#include "Widgets/Input/SFilePathPicker.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "FImgMediaFilePathCustomization"

/* IPropertyTypeCustomization interface
 *****************************************************************************/

TSharedRef<IPropertyTypeCustomization> FImgMediaFilePathCustomization::MakeInstance()
{
	return MakeShared<FImgMediaFilePathCustomization>();
}

void FImgMediaFilePathCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle,
	FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FImgMediaFilePathCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PathStringProperty = StructPropertyHandle->GetChildHandle("FilePath");

	// Create a name for this setting from the owner name and the property name.
	FProperty* Property = StructPropertyHandle->GetProperty();
	TObjectPtr<UObject> OwnerObject = Property->GetOwnerUObject();
	ConfigSettingName = Property->GetName();
	if (OwnerObject != nullptr)
	{
		ConfigSettingName = OwnerObject->GetName() + ConfigSettingName;
	}
	
	// Set filter.
	FString FileTypeFilter = TEXT("All files (*.*)|*.*");

	// Get the last value for this setting.
	FString PickedPath;
	GConfig->GetString(TEXT("ImgMedia.Settings"), *ConfigSettingName, PickedPath, GEditorPerProjectIni);
	PathStringProperty->SetValue(PickedPath);

	// Add file picker.
	ChildBuilder.AddCustomRow(LOCTEXT("TimeLabel", "Time"))
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(0.0f)
		.MinDesiredWidth(125.0f)
		[
			SNew(SFilePathPicker)
				.BrowseButtonImage(FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis"))
				.BrowseButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.BrowseButtonToolTip(LOCTEXT("FileButtonToolTipText", "Choose a file from this computer"))
				.BrowseDirectory_Lambda([this]()->FString
				{
					return FPaths::GetPath(HandleFilePathPickerFilePath());
				})
				.BrowseTitle(LOCTEXT("PropertyEditorTitle", "File picker..."))
				.FilePath(this, &FImgMediaFilePathCustomization::HandleFilePathPickerFilePath)
				.FileTypeFilter(FileTypeFilter)
				.OnPathPicked(this, &FImgMediaFilePathCustomization::HandleFilePathPickerPathPicked)
		];
}

FString FImgMediaFilePathCustomization::HandleFilePathPickerFilePath() const
{
	FString FilePath;
	PathStringProperty->GetValue(FilePath);

	return FilePath;
}

void FImgMediaFilePathCustomization::HandleFilePathPickerPathPicked(const FString& PickedPath)
{
	PathStringProperty->SetValue(PickedPath);
	GConfig->SetString(TEXT("ImgMedia.Settings"), *ConfigSettingName, *PickedPath, GEditorPerProjectIni);
}


#undef LOCTEXT_NAMESPACE
