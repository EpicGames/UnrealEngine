// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IPropertyTypeCustomization.h"

class IPropertyHandle;

/**
 * Implements a customization for the FFilePath class.
 * 
 * This will preserve the file path when closing.
 */
class FImgMediaFilePathCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	/**
	 * Gets the file path.
	 */
	FString HandleFilePathPickerFilePath() const;

	/**
	 * Called when a path is selected.
	 */
	void HandleFilePathPickerPathPicked(const FString& PickedPath);

	/** Pointer to the string that will be seet when changing the path */
	TSharedPtr<IPropertyHandle> PathStringProperty;

	/** Name to save the file path to. */
	FString ConfigSettingName;
};

