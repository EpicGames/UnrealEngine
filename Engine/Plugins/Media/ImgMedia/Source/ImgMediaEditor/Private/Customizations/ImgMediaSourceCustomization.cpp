// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImgMediaSourceCustomization.h"

#include "IMediaModule.h"
#include "ImgMediaEditorModule.h"
#include "ImgMediaSource.h"
#include "GameFramework/Actor.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"

#include "Styling/CoreStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SDirectoryPicker.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SFilePathPicker.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IDetailPropertyRow.h"


#define LOCTEXT_NAMESPACE "FImgMediaSourceCustomization"


/* IDetailCustomization interface
 *****************************************************************************/

void FImgMediaSourceCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyHandle = InPropertyHandle;

	TWeakPtr<FImgMediaSourceCustomization> WeakSelf = StaticCastWeakPtr<FImgMediaSourceCustomization>(AsWeak());
	
	// customize 'File' category
	HeaderRow
		.NameContent()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text(LOCTEXT("SequencePathPropertyName", "Sequence Path"))
							.ToolTipText(GetSequencePathProperty(PropertyHandle)->GetToolTipText())
					]

				+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SImage)
							.Image(FCoreStyle::Get().GetBrush("Icons.Warning"))
							.ToolTipText(LOCTEXT("SequencePathWarning", "The selected image sequence will not get packaged, because its path points to a directory outside the project's /Content/Movies/ directory."))
							.Visibility(this, &FImgMediaSourceCustomization::HandleSequencePathWarningIconVisibility)
					]
			]
		.ValueContent()
			.MaxDesiredWidth(0.0f)
			.MinDesiredWidth(125.0f)
			[
				SNew(SFilePathPicker)
					.BrowseButtonImage(FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis"))
					.BrowseButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.BrowseButtonToolTip(LOCTEXT("SequencePathBrowseButtonToolTip", "Choose a file from this computer"))
					.DialogReturnsFullPath(true)
					.BrowseDirectory_Lambda([WeakSelf]() -> FString
					{
						const TSharedPtr<FImgMediaSourceCustomization> Self = WeakSelf.Pin();
						const FString SequencePath = Self ? Self->GetSequenceFullPath() : TEXT("");
						return SequencePath.IsEmpty() ? (FPaths::ProjectContentDir() / TEXT("Movies")) : SequencePath;
					})
					.FilePath_Lambda([WeakSelf]() -> FString
					{
						const TSharedPtr<FImgMediaSourceCustomization> Self = WeakSelf.Pin();
						return Self ? Self->GetSequencePath() : TEXT("");
					})
					.FileTypeFilter_Lambda([]() -> FString
					{
						return TEXT("All files (*.*)|*.*|EXR files (*.exr)|*.exr");
					})
					.OnPathPicked(this, &FImgMediaSourceCustomization::HandleSequencePathPickerPathPicked)
					.ToolTipText(LOCTEXT("SequencePathToolTip", "The path to an image sequence file on this computer"))
			];
}

void FImgMediaSourceCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

/* FImgMediaSourceCustomization implementation
 *****************************************************************************/

FString FImgMediaSourceCustomization::GetSequencePathFromChildProperty(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
{
	FString FilePath;
	TSharedPtr<IPropertyHandle> SequencePathProperty = GetSequencePathPathProperty(InPropertyHandle);
	if (SequencePathProperty.IsValid())
	{
		if (SequencePathProperty->GetValue(FilePath) != FPropertyAccess::Success)
		{
			UE_LOGF(LogImgMediaEditor, Error, "FImgMediaSourceCustomization could not get SequencePath.");
		}
	}

	return FilePath;
}

FString FImgMediaSourceCustomization::GetSequencePath() const
{
	return GetSequencePathFromChildProperty(PropertyHandle);
}

FString FImgMediaSourceCustomization::GetSequenceFullPath() const
{
	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);
	for (UObject* OuterObject : OuterObjects)
	{
		if (const UImgMediaSource* Source = Cast<UImgMediaSource>(OuterObject))
		{
			// Using the media source to resolve the full path.
			return Source->GetFullPath();
		}
	}
	return FString();
}

TSharedPtr<IPropertyHandle> FImgMediaSourceCustomization::GetSequencePathProperty(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
{
	TSharedPtr<IPropertyHandle> SequencePathProperty;

	if ((InPropertyHandle.IsValid()) && (InPropertyHandle->IsValidHandle()))
	{
		TSharedPtr<IPropertyHandle> ParentHandle = InPropertyHandle->GetParentHandle();
		if (ParentHandle.IsValid())
		{
			SequencePathProperty = ParentHandle->GetChildHandle("SequencePath");
		}
	}

	return SequencePathProperty;
}

TSharedPtr<IPropertyHandle> FImgMediaSourceCustomization::GetSequencePathPathProperty(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
{
	TSharedPtr<IPropertyHandle> SequencePathPathProperty;

	TSharedPtr<IPropertyHandle> SequencePathProperty = GetSequencePathProperty(InPropertyHandle);
	if (SequencePathProperty.IsValid())
	{
		SequencePathPathProperty = SequencePathProperty->GetChildHandle("Path");
	}

	return SequencePathPathProperty;
}

/* FImgMediaSourceCustomization callbacks
 *****************************************************************************/

void FImgMediaSourceCustomization::HandleSequencePathPickerPathPicked(const FString& PickedPath)
{
	// The user may have put a path that is relative to the project, or to the content, or to the running process.
	const FString SanitizedPickedPath = UImgMediaSource::SanitizeTokenizedSequencePath(PickedPath);

	// update property
	const TSharedPtr<IPropertyHandle> SequencePathPathProperty = GetSequencePathPathProperty(PropertyHandle);
	if (SequencePathPathProperty.IsValid())
	{
		if (SequencePathPathProperty->SetValue(SanitizedPickedPath) != FPropertyAccess::Success)
		{
			UE_LOGF(LogImgMediaEditor, Error, "FImgMediaSourceCustomization could not set SequencePath.");
		}
	}
}

EVisibility FImgMediaSourceCustomization::HandleSequencePathWarningIconVisibility() const
{
	const FString SequenceFullPath = GetSequenceFullPath();

	if (SequenceFullPath.IsEmpty())
	{
		return EVisibility::Hidden;
	}

	const FString FullMoviesPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir() / TEXT("Movies"));
	if (FPaths::IsUnderDirectory(SequenceFullPath, FullMoviesPath))
	{
		if (FPaths::DirectoryExists(SequenceFullPath))
		{
			return EVisibility::Hidden;
		}

		// Path doesn't exist
		return EVisibility::Visible;
	}

	// Sequence path is not inside Movies folder
	return EVisibility::Visible;
}


#undef LOCTEXT_NAMESPACE
