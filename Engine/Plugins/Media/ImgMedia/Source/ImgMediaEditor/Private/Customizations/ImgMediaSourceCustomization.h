// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"

class IDetailCategoryBuilder;
class IDetailGroup;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SEditableTextBox;

/**
 * Implements a details view customization for the UImgMediaSource class.
 */
class FImgMediaSourceCustomization
	: public IPropertyTypeCustomization
{
public:

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

public:

	/**
	 * Creates an instance of this class.
	 *
	 * @return The new instance.
	 */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FImgMediaSourceCustomization());
	}

	/**
	 * Get the path to the currently selected image sequence.
	 *
	 * @param InPropertyHandle A property that is a child of the top level ImgMediaSource property.
	 * 
	 * @return Sequence path string.
	 */
	static FString GetSequencePathFromChildProperty(const TSharedPtr<IPropertyHandle>& InPropertyHandle);

protected:

	/**
	 * Get the path to the currently selected image sequence.
	 *
	 * @return Sequence path string.
	 */
	FString GetSequencePath() const;

	/**
	 *  Returns the fully resolved path to the sequence directory of the currently selected media source.
	 */
	FString GetSequenceFullPath() const;

private:

	/** Callback for picking a path in the source directory picker. */
	void HandleSequencePathPickerPathPicked(const FString& PickedPath);

	/** Callback for getting the visibility of warning icon for invalid SequencePath paths. */
	EVisibility HandleSequencePathWarningIconVisibility() const;

	/** Returns the property for SequencePath. */
	static TSharedPtr<IPropertyHandle> GetSequencePathProperty(const TSharedPtr<IPropertyHandle>& InPropertyHandle);
	
	/** Returns the property for SequencePath->Path. */
	static TSharedPtr<IPropertyHandle> GetSequencePathPathProperty(const TSharedPtr<IPropertyHandle>& InPropertyHandle);
	
private:

	/** Text block widget showing the found proxy directories. */
	TSharedPtr<SEditableTextBox> ProxiesTextBlock;

	/** Stores our property. */
	TSharedPtr<IPropertyHandle> PropertyHandle;
};
