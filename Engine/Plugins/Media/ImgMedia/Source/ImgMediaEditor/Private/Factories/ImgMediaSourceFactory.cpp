// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImgMediaSourceFactory.h"
#include "Misc/Paths.h"
#include "ImgMediaSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImgMediaSourceFactory)


/* UExrFileMediaSourceFactory structors
 *****************************************************************************/

UImgMediaSourceFactory::UImgMediaSourceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// note: conflicts with .exr image import by TextureFactory in the import dialog
	// if you filter for *.exr it chooses the first extension alphabetically
	Formats.Add(TEXT("exr;EXR ImgMedia Image Sequence"));

	SupportedClass = UImgMediaSource::StaticClass();
	bEditorImport = true;
	
	// Required to allow texture factory to take priority when importing new image files
	ImportPriority = DefaultImportPriority - 1;
}


/* UFactory overrides
 *****************************************************************************/

UObject* UImgMediaSourceFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	UImgMediaSource* MediaSource = NewObject<UImgMediaSource>(InParent, InClass, InName, Flags);
	MediaSource->SetSequencePath(FPaths::GetPath(CurrentFilename));

	return MediaSource;
}

