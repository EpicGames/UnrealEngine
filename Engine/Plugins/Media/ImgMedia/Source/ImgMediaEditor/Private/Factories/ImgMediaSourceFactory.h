// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "ImgMediaSourceFactory.generated.h"

/**
 * Implements a factory for UImgMediaSource objects.
 */
UCLASS(hidecategories=Object)
class UImgMediaSourceFactory
	: public UFactory
{
	GENERATED_UCLASS_BODY()

public:

	//~ UFactory Interface

	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
};
