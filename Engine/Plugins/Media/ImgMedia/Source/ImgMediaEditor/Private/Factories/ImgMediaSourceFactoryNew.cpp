// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImgMediaSourceFactoryNew.h"

#include "AssetTypeCategories.h"
#include "ImgMediaSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImgMediaSourceFactoryNew)


/* UImgMediaSourceFactoryNew structors
 *****************************************************************************/

UImgMediaSourceFactoryNew::UImgMediaSourceFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UImgMediaSource::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory overrides
 *****************************************************************************/

UObject* UImgMediaSourceFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UImgMediaSource>(InParent, InClass, InName, Flags);
}


uint32 UImgMediaSourceFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool UImgMediaSourceFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}

