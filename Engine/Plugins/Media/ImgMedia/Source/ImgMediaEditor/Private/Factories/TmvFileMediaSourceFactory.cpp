// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvFileMediaSourceFactory.h"

#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "FileMediaSource.h"
#include "ITmvMediaModule.h"
#include "Containers/UnrealString.h"
#include "Decoder/ITmvMediaDemuxerFactory.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TmvFileMediaSourceFactory)


/* UTmvFileMediaSourceFactory structors
 *****************************************************************************/

UTmvFileMediaSourceFactory::UTmvFileMediaSourceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UFileMediaSource::StaticClass();
	bEditorImport = true;

	// Try now (may be empty if demuxer factories haven't been registered yet).
	InitializeFormats();

	// For the CDO, also register for PostEngineInit to retry once all modules are loaded.
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddWeakLambda(this, [this]()
		{
			InitializeFormats();
		});
	}
}

void UTmvFileMediaSourceFactory::InitializeFormats()
{
	const ITmvMediaModule* TmvMediaModule = ITmvMediaModule::Get();
	if (!TmvMediaModule)
	{
		return;
	}

	Formats.Reset();

	TArray<TWeakPtr<ITmvMediaDemuxerFactory, ESPMode::ThreadSafe>> DemuxerFactories;
	TmvMediaModule->GetDemuxerFactories(DemuxerFactories);
	for (const TWeakPtr<ITmvMediaDemuxerFactory, ESPMode::ThreadSafe>& DemuxerFactoryWeak : DemuxerFactories)
	{
		if (const TSharedPtr<ITmvMediaDemuxerFactory> DemuxerFactory = DemuxerFactoryWeak.Pin())
		{
			for (const FString& Extension : DemuxerFactory->GetSupportedContainerFormats())
			{
				Formats.Add(FString::Printf(TEXT("%s;%s"), *Extension, *DemuxerFactory->GetDisplayName().ToString()));
			}
		}
	}
}


/* UFactory overrides
 *****************************************************************************/

bool UTmvFileMediaSourceFactory::FactoryCanImport(const FString& InFilename)
{
	if (const ITmvMediaModule* TmvMediaModule = ITmvMediaModule::Get())
	{
		const FString Extension = FPaths::GetExtension(InFilename);
		if (TmvMediaModule->FindDemuxerFactoryForExtension(Extension).IsValid())
		{
			return true;
		}
	}
	return false;
}


UObject* UTmvFileMediaSourceFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, const FString& InFilename, const TCHAR* InParms, FFeedbackContext* InWarn, bool& bOutOperationCanceled)
{
	UFileMediaSource* MediaSource = NewObject<UFileMediaSource>(InParent, InClass, InName, InFlags);
	MediaSource->SetFilePath(CurrentFilename);

	return MediaSource;
}

