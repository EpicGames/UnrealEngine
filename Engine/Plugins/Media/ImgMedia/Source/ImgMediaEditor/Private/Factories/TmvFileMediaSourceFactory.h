// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"

#include "TmvFileMediaSourceFactory.generated.h"

/**
 * Implements a factory for importing Tmv container files as UFileMediaSource objects.
 */
UCLASS(hidecategories=Object)
class UTmvFileMediaSourceFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UFactory
	virtual bool FactoryCanImport(const FString& InFilename) override;
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, const FString& InFilename, const TCHAR* InParms, FFeedbackContext* InWarn, bool& bOutOperationCanceled) override;
	//~ End UFactory

private:
	/** Populate the Formats array from registered demuxer factories. Safe to call multiple times. */
	void InitializeFormats();
};
