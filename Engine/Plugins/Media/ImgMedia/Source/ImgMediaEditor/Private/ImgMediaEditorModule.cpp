// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImgMediaEditorModule.h"
#include "Modules/ModuleManager.h"

#include "ContentBrowserMenuContexts.h"
#include "Customizations/ImgMediaSourceCustomization.h"
#include "Framework/Application/SlateApplication.h"
#include "ImgMediaEditorModule.h"
#include "IImgMediaModule.h"
#include "ImgMediaSource.h"
#include "PropertyEditorModule.h"
#include "ToolMenus.h"
#include "UObject/NameTypes.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SImgMediaBandwidth.h"
#include "Widgets/SImgMediaCache.h"
#include "Widgets/SImgMediaProcessEXR.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "ImgMediaEditorModule"

DEFINE_LOG_CATEGORY(LogImgMediaEditor);

static const FName ImgMediaBandwidthTabName(TEXT("ImgMediaBandwidth"));
static const FName ImgMediaCacheTabName(TEXT("ImgMediaCache"));

/**
 * Implements the ImgMediaEditor module.
 */
class FImgMediaEditorModule
	: public IImgMediaEditorModule
{
public:
	//~ IImgMediaEditorModule interface
	const TArray<TWeakPtr<FImgMediaPlayer>>& GetMediaPlayers()
	{
		return MediaPlayers;
	}

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		RegisterCustomizations();
		RegisterTabSpawners();
		ExtendContentMenu();

		IImgMediaModule* ImgMediaModule = FModuleManager::LoadModulePtr<IImgMediaModule>("ImgMedia");
		if (ImgMediaModule != nullptr)
		{
			ImgMediaModule->OnImgMediaPlayerCreated.AddRaw(this, &FImgMediaEditorModule::OnImgMediaPlayerCreated);

		}
	}

	virtual void ShutdownModule() override
	{
		UnregisterTabSpawners();
		UnregisterCustomizations();
	}

protected:

	/** Register details view customizations. */
	void RegisterCustomizations()
	{
		CustomizedStructName = FImgMediaSourceCustomizationSequenceProxy::StaticStruct()->GetFName();

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		{
#if WITH_EDITORONLY_DATA
			PropertyModule.RegisterCustomPropertyTypeLayout(CustomizedStructName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FImgMediaSourceCustomization::MakeInstance));
#endif // WITH_EDITORONLY_DATA
		}
	}

	/** Unregister details view customizations. */
	void UnregisterCustomizations()
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		{
#if WITH_EDITORONLY_DATA
			PropertyModule.UnregisterCustomPropertyTypeLayout(CustomizedStructName);
#endif // WITH_EDITORONLY_DATA
		}
	}

	void RegisterTabSpawners()
	{
		const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();
		TSharedRef<FWorkspaceItem> MediaBrowserGroup = MenuStructure.GetLevelEditorMediaCategory();

		// Add bandwidth tab.
		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(ImgMediaBandwidthTabName,
			FOnSpawnTab::CreateStatic(&FImgMediaEditorModule::SpawnBandwidthTab))
			.SetGroup(MediaBrowserGroup)
			.SetDisplayName(LOCTEXT("ImgMediaBandwidthMonitorTabTitle", "Bandwidth Monitor"))
			.SetTooltipText(LOCTEXT("ImgMediaBandwidthMonitorTooltipText", "Open the bandwidth monitor tab."))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "SequenceRecorder.TabIcon"));

		// Add cache tab.
		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(ImgMediaCacheTabName,
			FOnSpawnTab::CreateStatic(&FImgMediaEditorModule::SpawnCacheTab))
			.SetGroup(MediaBrowserGroup)
			.SetDisplayName(LOCTEXT("ImgMediaGlobalCacheTabTitle", "Global Cache"))
			.SetTooltipText(LOCTEXT("ImgMediaGlobalCacheTooltipText", "Open the global cache tab."))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "SequenceRecorder.TabIcon"));

		// Add process images tab.
		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(ImgMediaProcessEXRTabName,
			FOnSpawnTab::CreateStatic(&FImgMediaEditorModule::SpawnProcessEXRTab))
			.SetGroup(MediaBrowserGroup)
			.SetDisplayName(LOCTEXT("ImgMediaProcessEXRTabTitle", "Process EXR"))
			.SetTooltipText(LOCTEXT("ImgMediaProcessEXRTooltipText", "Open the Process EXR tab."))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));
	}

	void UnregisterTabSpawners()
	{
		if (FSlateApplication::IsInitialized())
		{
			FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(ImgMediaProcessEXRTabName);
			FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(ImgMediaCacheTabName);
			FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(ImgMediaBandwidthTabName);
		}
	}

	static TSharedRef<SDockTab> SpawnBandwidthTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SNew(SImgMediaBandwidth)
			];
	}

	static TSharedRef<SDockTab> SpawnCacheTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SNew(SImgMediaCache)
			];
	}

	static TSharedRef<SDockTab> SpawnProcessEXRTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		// Create tab.
		TSharedPtr< SImgMediaProcessEXR> ProcessEXR;
		TSharedRef <SDockTab> Tab = SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SAssignNew(ProcessEXR, SImgMediaProcessEXR)
			];

		// Override input path if desired.
		if (ProcessEXRInputPath.IsEmpty() == false)
		{
			ProcessEXR->SetInputPath(ProcessEXRInputPath);
			ProcessEXRInputPath.Empty();
		}

		return Tab;
	}

	void ExtendContentMenu()
	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.ImgMediaSource");
		
		FToolMenuSection& Section = Menu->AddDynamicSection("ImgMediaSource", FNewToolMenuDelegate::CreateLambda([this](UToolMenu* ToolMenu)
		{
			if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(*ToolMenu))
			{
				if (Context->SelectedAssets.Num() != 1)
				{
					return;
				}

				// Add menu entry.
				if (UImgMediaSource* ImgMediaSource = Cast<UImgMediaSource>(Context->SelectedAssets[0].GetAsset()))
				{
					// Get the full path from media source.
					// Make sure it ends with / so it opens the directory.
					FString InputPath = ImgMediaSource->GetFullPath();
					if (InputPath.EndsWith(TEXT("/")) == false)
					{
						InputPath += TEXT("/");
					}

					const TAttribute<FText> Label = LOCTEXT("ProcessEXR", "Process EXRs");
					const TAttribute<FText> ToolTip = LOCTEXT("ProcessEXR_Tooltip", "Open a tab to allow adding tiles and mips to an EXR sequence.");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(),
						"LevelEditor.Tabs.Viewports");
					const FToolMenuExecuteAction UIAction =
						FToolMenuExecuteAction::CreateLambda([InputPath]
						(const FToolMenuContext& InContext)
					{
						ProcessEXRInputPath = InputPath;
						OpenProcesEXRTab();
					});
				
					const FName SectionName = TEXT("ImgMedia");
					FToolMenuSection* Section = ToolMenu->FindSection(SectionName);
					if (!Section)
					{
						Section = &(ToolMenu->AddSection(SectionName, LOCTEXT("ImgMediaSectionLabel", "ImgMedia")));
					}

					Section->AddMenuEntry("ImgMedia_ProcessEXR", Label, ToolTip, Icon, UIAction);
				}
			}
		}), FToolMenuInsert(NAME_None, EToolMenuInsertType::Default));
	}

	/**
	 * Call this to open the tab to process EXRs.
	 */
	static void OpenProcesEXRTab()
	{
		FTabId TabID(ImgMediaProcessEXRTabName);
		FGlobalTabmanager::Get()->TryInvokeTab(TabID);
	}

	void OnImgMediaPlayerCreated(const TSharedPtr<FImgMediaPlayer>& Player)
	{
		// Try and replace an expired player.
		bool bIsAdded = false;
		for (TWeakPtr<FImgMediaPlayer>& PlayerPointer : MediaPlayers)
		{
			if (PlayerPointer.IsValid() == false)
			{
				PlayerPointer = Player;
				bIsAdded = true;
				break;
			}
		}

		// If we were not able to add it, just add it now.
		if (bIsAdded == false)
		{
			MediaPlayers.Add(Player);
		}

		// Send out the message.
		OnImgMediaEditorPlayersUpdated.Broadcast();
	}

private:

	/** Customization name to avoid reusing staticstruct during shutdown. */
	FName CustomizedStructName;

	/** Names for tabs. */
	static FLazyName ImgMediaProcessEXRTabName;

	/** This will be passed to ProceessEXR and then cleared. */
	static FString ProcessEXRInputPath;

	/** Array of all our players. */
	TArray<TWeakPtr<FImgMediaPlayer>> MediaPlayers;
};

FLazyName FImgMediaEditorModule::ImgMediaProcessEXRTabName(TEXT("ImgMediaProcessEXR"));
FString FImgMediaEditorModule::ProcessEXRInputPath;

IMPLEMENT_MODULE(FImgMediaEditorModule, ImgMediaEditor);

#undef LOCTEXT_NAMESPACE
