// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"

class FImgMediaPlayer;

/** Log category for this module. */
DECLARE_LOG_CATEGORY_EXTERN(LogImgMediaEditor, Log, All);

/** Callback for when the list of players is updated. */
DECLARE_MULTICAST_DELEGATE(FOnImgMediaEditorPlayersUpdated);


/**
* Interface for the ImgMediaEditor module.
*/
class IImgMediaEditorModule
	: public IModuleInterface
{
public:

	/**
	 * Call this to get a list of all the players. *
	 */
	virtual const TArray<TWeakPtr<FImgMediaPlayer>>& GetMediaPlayers() = 0;

	/** Add to this to get a callback when the list of players gets updated. */
	FOnImgMediaEditorPlayersUpdated OnImgMediaEditorPlayersUpdated;
};
