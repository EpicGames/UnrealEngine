// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Contains options for processing image sequences.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ImgMediaProcessEXROptions.generated.h"

UCLASS(MinimalAPI)
class UImgMediaProcessEXROptions : public UObject
{
	GENERATED_UCLASS_BODY()
public:
	/** The directory that contains the image sequence files. */
	UPROPERTY(EditAnywhere, Category = Sequence)
	FFilePath InputPath;

	/** The directory to output the processed image sequence files to. */
	UPROPERTY(EditAnywhere, Category = Sequence)
	FDirectoryPath OutputPath;

	/** If checked, then enable mip mapping. */
	UPROPERTY(EditAnywhere, Category = Sequence)
	bool bEnableMipMapping = true;

	/** If checked, then enable tiling. */
	UPROPERTY(EditAnywhere, Category = Tiles)
	bool bEnableTiling = true;
	
	/** Width of a tile in pixels. If 0, then do not make tiles. */
	UPROPERTY(EditAnywhere, Transient, Category = Tiles, meta = (EditCondition = "bEnableTiling", ClampMin = "0.0"))
	int32 TileSizeX = 256;

	/** Height of a tile in pixels. If 0, then do not make tiles. */
	UPROPERTY(EditAnywhere, Transient, Category = Tiles, meta = (EditCondition = "bEnableTiling", ClampMin = "0.0"))
	int32 TileSizeY = 256;

	/** Number of threads to use when processing. If 0 then this number is set automatically. */
	UPROPERTY(EditAnywhere, Transient, Category = Processing, meta = (ClampMin = "0.0"))
	int32 NumThreads = 0;

	/** Use a player to read in the image. */
	UPROPERTY(EditAnywhere, Transient, Category = Processing)
	bool bUsePlayer = false;

	/** 
	* This option removes alpha channel. Since alpha channel adds 25% reading cost, it is suggested to have alpha channel removed if it is not used. 
	*/
	UPROPERTY(EditAnywhere, Category = Channels)
	bool bRemoveAlphaChannel = false;

	/** Tint each mip level a different colour to help with debugging. */
	UPROPERTY(EditAnywhere, Transient, Category = Debug)
	bool bEnableMipLevelTint = false;

	/** Colour to tint each mip level. */
	UPROPERTY(EditAnywhere, Transient, Category = Debug)
	TArray<FLinearColor> MipLevelTints;
};
