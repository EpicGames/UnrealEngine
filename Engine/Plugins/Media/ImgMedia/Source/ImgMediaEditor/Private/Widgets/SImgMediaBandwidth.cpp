// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SImgMediaBandwidth.h"
#include "DetailLayoutBuilder.h"
#include "IImgMediaModule.h"
#include "ImgMediaEditorModule.h"
#include "Loader/ImgMediaLoader.h"
#include "Modules/ModuleManager.h"
#include "Player/ImgMediaPlayer.h"
#include "SlateOptMacros.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ImgMediaBandwidth"

constexpr double SImgMediaBandwidth::BANDWIDTH_REFRESH_RATE;

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SImgMediaBandwidth::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)

		// Bandwidth label.
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 5, 0, 0)
		[
			SNew(STextBlock)
				.Text(LOCTEXT("Bandwidth", "Bandwidth"))
		]

		// Container for all players.
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(PlayersContainer, SVerticalBox)
			]
		
	];

	// Update players.
	RefreshPlayersContainer();

	// Get notified when a player is created.
	IImgMediaEditorModule* ImgMediaEditorModule = FModuleManager::LoadModulePtr<IImgMediaEditorModule>("ImgMediaEditor");
	if (ImgMediaEditorModule != nullptr)
	{
		ImgMediaEditorModule->OnImgMediaEditorPlayersUpdated.AddSP(this, &SImgMediaBandwidth::RefreshPlayersContainer);
	}

}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SImgMediaBandwidth::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Call parent.
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (InCurrentTime - LastBandwidthUpdateTime >= BANDWIDTH_REFRESH_RATE)
	{
		// Loop over all players.
		for (const FPlayerInfo& PlayerInfo : PlayerInfos)
		{
			TSharedPtr< FImgMediaPlayer> Player = PlayerInfo.Player.Pin();
			if (Player != nullptr)
			{
				// Update widgets with info.
				// Set URL.
				PlayerInfo.UrlTextBlock->SetText(FText::Format(
					LOCTEXT("Url", "URL: {0}"),
					FText::FromString(Player->GetUrl())));

				// Get the loader.
				TSharedPtr<FImgMediaLoader, ESPMode::ThreadSafe> Loader = Player->GetLoader();
				if (Loader.IsValid())
				{
					// Set current bandwidth.
					float Bandwidth = Loader->GetCurrentBandwidth() / (1000.0f * 1000.0f);
					PlayerInfo.CurrentBandwidthTextBlock->SetText(FText::Format(
						LOCTEXT("Current", "Current: {0} MB/s"),
						FText::AsNumber(Bandwidth)));

					// Set effective bandwidth.
					Bandwidth = Loader->GetEffectiveBandwidth() / (1000.0f * 1000.0f);
					PlayerInfo.EffectiveBandwidthTextBlock->SetText(FText::Format(
						LOCTEXT("Effective", "Effective: {0} MB/s"),
						FText::AsNumber(Bandwidth)));

					// Set required bandwidth.
					Bandwidth = Loader->GetRequiredBandwidth() / (1000.0f * 1000.0f);
					PlayerInfo.RequiredBandwidthTextBlock->SetText(FText::Format(
						LOCTEXT("Required", "Required: {0} MB/s"),
						FText::AsNumber(Bandwidth)));
				}
			}
		}

		LastBandwidthUpdateTime = InCurrentTime;
	}
}

void SImgMediaBandwidth::RefreshPlayersContainer()
{
	// Reset our players.
	PlayersContainer->ClearChildren();
	PlayerInfos.Reset();

	IImgMediaEditorModule* ImgMediaEditorModule = FModuleManager::LoadModulePtr<IImgMediaEditorModule>("ImgMediaEditor");
	if (ImgMediaEditorModule != nullptr)
	{
		const TArray<TWeakPtr<FImgMediaPlayer>>& MediaPlayers = ImgMediaEditorModule->GetMediaPlayers();

		// Loop over all players.
		for (TWeakPtr<FImgMediaPlayer> PlayerWeakPtr : MediaPlayers)
		{
			TSharedPtr< FImgMediaPlayer> Player = PlayerWeakPtr.Pin();
			if (Player != nullptr)
			{
				FPlayerInfo PlayerInfo;
				PlayerInfo.Player = Player;
				PlayersContainer->AddSlot()
					.AutoHeight()
					.Padding(0, 5, 0, 0)
					[
						SNew(SVerticalBox)

						// Add URL.
						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0, 5, 0, 0)
							[
								SAssignNew(PlayerInfo.UrlTextBlock, STextBlock)
							]

						// Add bandwidth.
						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0, 5, 0, 0)
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
									.FillWidth(1.0f)
									[
										SAssignNew(PlayerInfo.CurrentBandwidthTextBlock, STextBlock)
									]

								+ SHorizontalBox::Slot()
									.FillWidth(1.0f)
									[
										SAssignNew(PlayerInfo.EffectiveBandwidthTextBlock, STextBlock)
									]

								+ SHorizontalBox::Slot()
									.FillWidth(1.0f)
									[
										SAssignNew(PlayerInfo.RequiredBandwidthTextBlock, STextBlock)
									]
							]

					];

				// Add this to the list.
				PlayerInfos.Add(PlayerInfo);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
