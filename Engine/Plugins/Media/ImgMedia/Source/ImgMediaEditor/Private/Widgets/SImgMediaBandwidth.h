// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"

class FImgMediaPlayer;
class STextBlock;
class SVerticalBox;

/**
 * SImgMediaCache manages caching for image sequences.
 */
class SImgMediaBandwidth : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SImgMediaBandwidth){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	//~ SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	/**
	 * Update our list of players and the widgets accordingly.
	 */
	void RefreshPlayersContainer();

	/** Info for a single player. */
	struct FPlayerInfo
	{
		TWeakPtr<FImgMediaPlayer> Player;
		TSharedPtr<STextBlock> UrlTextBlock;
		TSharedPtr<STextBlock> CurrentBandwidthTextBlock;
		TSharedPtr<STextBlock> EffectiveBandwidthTextBlock;
		TSharedPtr<STextBlock> RequiredBandwidthTextBlock;
	};

	/** Array of all players. */
	TArray<FPlayerInfo> PlayerInfos;
	/** Widget to hold all widgets for all the players. */
	TSharedPtr<SVerticalBox> PlayersContainer;
	/** Timestamp to slow down bandwidth updates. */
	double LastBandwidthUpdateTime = 0.0;
	/** Bandwidth refresh rate in seconds. */
	static constexpr double BANDWIDTH_REFRESH_RATE = 0.25;
};
