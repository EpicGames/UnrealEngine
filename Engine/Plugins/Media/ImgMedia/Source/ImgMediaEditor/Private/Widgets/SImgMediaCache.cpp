// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SImgMediaCache.h"
#include "DetailLayoutBuilder.h"
#include "IImgMediaModule.h"
#include "ImgMediaGlobalCache.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ImgMediaCache"

SImgMediaCache::~SImgMediaCache()
{
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SImgMediaCache::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.Padding(0, 20, 0, 0)
			.AutoHeight()
		
		// Global cache label.
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 5, 0, 0)
		[
			SNew(STextBlock)
				.Text(LOCTEXT("GlobalCache", "Global Cache"))
		]

		// Global cache info.
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SGridPanel)

				+ SGridPanel::Slot(0, 0)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CurrentSize", "Current Size"))
					]
				+ SGridPanel::Slot(1, 0)
					.Padding(2.0f)
					[
						SAssignNew(CurrentSizeTextBlock, STextBlock)
						.Text(FText::GetEmpty())
					]
				+ SGridPanel::Slot(0, 1)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("MaxSize", "Max Size"))
					]
				+ SGridPanel::Slot(1, 1)
					.Padding(2.0f)
					[
						SAssignNew(MaxSizeTextBlock, STextBlock)
						.Text(FText::GetEmpty())
					]
			]
			
		// Clear global cache.
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4.0f)
			.HAlign(HAlign_Left)
			[
				SNew(SButton)
					.OnClicked(this, &SImgMediaCache::OnClearGlobalCacheClicked)
					.Text(LOCTEXT("ClearGlobalCache", "Clear Global Cache"))
					.ToolTipText(LOCTEXT("ClearGlobalCacheButtonToolTip", "Clear the global cache."))
			]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SImgMediaCache::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Call parent.
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	// Get cache info.
	SIZE_T CurrentSize = 0;
	SIZE_T MaxSize = 0;
	FImgMediaGlobalCache* GlobalCache = IImgMediaModule::GetGlobalCache();
	if (GlobalCache != nullptr)
	{
		CurrentSize = GlobalCache->GetCurrentSize();
		MaxSize = GlobalCache->GetMaxSize();
	}

	// Update widgets.
	FText NumberText;
	NumberText = ConvertMemorySizeToText(CurrentSize);
	CurrentSizeTextBlock->SetText(NumberText);
	NumberText = ConvertMemorySizeToText(MaxSize);
	MaxSizeTextBlock->SetText(NumberText);
}

FReply SImgMediaCache::OnClearGlobalCacheClicked()
{
	// Clear the cache.
	FImgMediaGlobalCache* GlobalCache = IImgMediaModule::GetGlobalCache();
	if (GlobalCache != nullptr)
	{
		GlobalCache->EmptyCache();
	}

	return FReply::Handled();
}

FText SImgMediaCache::ConvertMemorySizeToText(SIZE_T Size)
{
	FText NumberText;

	if (Size > (100 * 1024 * 1024))
	{
		NumberText = FText::Format(LOCTEXT("SizeMB", "{0} MB"), (int32)(Size / (1024 * 1024)));
	}
	else if (Size > 100 * 1024)
	{
		NumberText = FText::Format(LOCTEXT("SizeKB", "{0} KB"), (int32)(Size / 1024));
	}
	else
	{
		NumberText = FText::Format(LOCTEXT("SizeB", "{0} B"), (int32)Size);
	}

	return NumberText;
}

#undef LOCTEXT_NAMESPACE
