// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"

class STextBlock;

/**
 * SImgMediaCache manages caching for image sequences.
 */
class SImgMediaCache : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SImgMediaCache){}
	SLATE_END_ARGS()

	virtual ~SImgMediaCache();

	void Construct(const FArguments& InArgs);

	// SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;


private:
	/** Called when we click on the clear global cache button. */
	FReply OnClearGlobalCacheClicked();

	/**
	 * Converts a number to a text with KB, MB, etc dependnig on the size.
	 *
	 * @param Size		Size in bytes.
	 * 
	 * @return Text representing the number.
	 */
	FText ConvertMemorySizeToText(SIZE_T Size);

	/** Widget to show curent size of cache. */
	TSharedPtr<STextBlock> CurrentSizeTextBlock;
	/** Widget to show max size of cache. */
	TSharedPtr<STextBlock> MaxSizeTextBlock;
};
