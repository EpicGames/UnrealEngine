// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SImgMediaProcessEXR.h"

#include "Async/Async.h"
#include "Customizations/ImgMediaFilePathCustomization.h"
#include "Editor.h"
#include "Engine/Canvas.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformFileManager.h"
#include "IImageWrapperModule.h"
#include "IImgMediaModule.h"
#include "IMediaEventSink.h"
#include "ImageUtils.h"
#include "ImageWrapperHelper.h"
#include "ImgMediaEditorModule.h"
#include "ImgMediaProcessEXROptions.h"
#include "Math/UnrealMathUtility.h"
#include "MediaHelpers.h"
#include "MediaPlayer.h"
#include "MediaSource.h"
#include "MediaTexture.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "PropertyEditorModule.h"
#include "RenderingThread.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "TextureResource.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/STextBlock.h"

#if IMGMEDIAEDITOR_EXR_SUPPORTED_PLATFORM
#include "OpenExrWrapper.h"
#endif // IMGMEDIAEDITOR_EXR_SUPPORTED_PLATFORM

#define LOCTEXT_NAMESPACE "ImgMediaProcessImages"

namespace UE::ImgMediaEditor::ProcessEXR
{
	/** Maximum wall-clock seconds to wait for the media player to deliver the next frame before aborting. */
	static TAutoConsoleVariable<float> CVarProgressTimeoutSeconds(
		TEXT("ImgMedia.ProcessEXR.ProgressTimeoutSeconds"),
		30.0f,
		TEXT("Wall-clock seconds the Process EXR widget will wait for the media player to deliver the next frame before aborting with an error. Set <= 0 to disable the watchdog."),
		ECVF_Default);

	/**
	 * Helper function to set the Notification text from a worker thread.
	 * @param InNotificationWeak Weak reference to the Notification object.
	 * @param InText Text to set.
	 */
	void SetNotificationTextSafe(const TWeakPtr<SNotificationItem>& InNotificationWeak, const FText& InText)
	{
		Async(EAsyncExecution::TaskGraphMainThread, [NotificationWeak = InNotificationWeak, Text = InText]()
		{
			if (TSharedPtr<SNotificationItem> Notification = NotificationWeak.Pin())
			{
				Notification->SetText(Text);
			}
		});
	}

	/**
	 * Helper function to set the Notification text from a worker thread.
	 * @param InNotification Reference to the Notification object.
	 * @param InText Text to set.
	 */
	void SetNotificationTextSafe(const TSharedPtr<SNotificationItem>& InNotification, const FText& InText)
	{
		if (InNotification.IsValid())
		{
			SetNotificationTextSafe(InNotification.ToWeakPtr(), InText);
		}
	}
}

SImgMediaProcessEXR::~SImgMediaProcessEXR()
{
	CleanUp();

	// Check if we have ongoing async threads preventing us from deleting the widget.
	if (NumAsyncTasks.load() > 0)
	{
		// Signal tasks to stop.
		bIsCancelling = true;

		// Wait for all async tasks to complete.
		while (NumAsyncTasks.load() > 0)
		{
			FPlatformProcess::Sleep(0.2f);
		}

		// Close the toaster message.
		if (ConfirmNotification.IsValid())
		{
			ConfirmNotification->SetEnabled(false);
			ConfirmNotification->SetCompletionState(SNotificationItem::CS_Fail);
			ConfirmNotification->ExpireAndFadeout();
			ConfirmNotification.Reset();
		}
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SImgMediaProcessEXR::Construct(const FArguments& InArgs)
{
	// Set up widgets.
	TSharedPtr<SBox> DetailsViewBox;
	
	ChildSlot
	[
		SNew(SScrollBox)

		// Add details view.
		+ SScrollBox::Slot()
			[
				SAssignNew(DetailsViewBox, SBox)
			]
			
		// Add process images button.
		+ SScrollBox::Slot()
			.Padding(8.0f)
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Right)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SAssignNew(StartButton, SButton)
							.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
							.IsEnabled_Lambda([this]
							{
								return (bIsProcessing == false) && (Options.IsValid()) &&
									(Options->OutputPath.Path.IsEmpty() == false);
							})
							.OnClicked(this, &SImgMediaProcessEXR::OnProcessImagesClicked)
							.ForegroundColor(FSlateColor::UseStyle())
							.Text(LOCTEXT("StartProcessImages", "Process Images"))
							.ToolTipText(LOCTEXT("StartProcesssImagesButtonToolTip", "Start processing images."))
					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SAssignNew(CancelButton, SButton)
							.OnClicked(this, &SImgMediaProcessEXR::OnCancelClicked)
							.Text(LOCTEXT("CancelProcessImages", "Cancel"))
							.ToolTipText(LOCTEXT("CancelProcesssImagesButtonToolTip", "Cancel processing images."))
					]
			]
	];
	bIsProcessing = false;
	bIsCancelling = false;
	UpdateWidgets();

	// Create object with our options.
	Options = TStrongObjectPtr<UImgMediaProcessEXROptions>(NewObject<UImgMediaProcessEXROptions>(GetTransientPackage(), NAME_None));

	// Create detail view with our options.
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->RegisterInstancedCustomPropertyTypeLayout(FName(TEXT("FilePath")),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(FImgMediaFilePathCustomization::MakeInstance));
	DetailsView->SetObject(Options.Get());

	DetailsViewBox->SetContent(DetailsView->AsShared());
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SImgMediaProcessEXR::SetInputPath(const FString& Path)
{
	if (Options.IsValid())
	{
		Options->InputPath.FilePath = Path;
	}
}

void SImgMediaProcessEXR::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bUsePlayer)
	{
		HandleProcessing();
	}
}

void SImgMediaProcessEXR::UpdateWidgets()
{
	CancelButton->SetEnabled(bIsProcessing && (!bIsCancelling));
}

FReply SImgMediaProcessEXR::OnProcessImagesClicked()
{
	if (bIsProcessing == false)
	{
		// Set that we are processing now.
		bIsProcessing = true;
		bUsePlayer = Options->bUsePlayer;
		UpdateWidgets();

		// Create notification.
		FNotificationInfo Info(FText::GetEmpty());
		Info.bFireAndForget = false;

		ConfirmNotification = FSlateNotificationManager::Get().AddNotification(Info);

		// Are we using a player?
		if (bUsePlayer)
		{
			GetGlobalImageParameters();

			// Create player.
			MediaPlayer = NewObject<UMediaPlayer>(GetTransientPackage(), "MediaPlayer", RF_Transient);
			MediaPlayer->SetLooping(true);
			MediaPlayer->PlayOnOpen = true;
			MediaPlayer->AddToRoot();
			MediaEventHandle = MediaPlayer->OnMediaEvent().AddRaw(this, &SImgMediaProcessEXR::HandleMediaPlayerEvent);

			// Create texture.
			MediaTexture = NewObject<UMediaTexture>(GetTransientPackage(), "MediaTexture", RF_Transient);
			MediaTexture->SetMediaPlayer(MediaPlayer);
			MediaTexture->UpdateResource();
			MediaTexture->AddToRoot();

			// Create media source.
			MediaSource = UMediaSource::SpawnMediaSourceForString(Options->InputPath.FilePath, GetTransientPackage());
			if (MediaSource == nullptr)
			{
				CancelWithError(FText::Format(
					LOCTEXT("FailedToSpawnMediaSource", "Failed to create a media source for \"{0}\". Verify that the path is valid and the file format is supported by an installed media player."),
					FText::FromString(Options->InputPath.FilePath)));
				return FReply::Handled();
			}
			MediaSource->AddToRoot();

			// Start playing.
			CurrentFrameIndex = 0;
			CurrentTime = FTimespan::FromSeconds(0.0f);
			LastProgressTimeSeconds = FPlatformTime::Seconds();
			MediaPlayer->SetBlockOnTimeRange(TRange<FTimespan>(CurrentTime,
				CurrentTime + FTimespan::FromSeconds(1.0f / 100000.0f)));
			MediaPlayer->OpenSource(MediaSource);
		}
		else
		{
			TWeakPtr<SImgMediaProcessEXR> WeakSelf = TSharedRef<SImgMediaProcessEXR>(AsShared(), this).ToWeakPtr();

			// Start async task to process files.
			Async(EAsyncExecution::Thread, [WeakSelf]()
			{
				// Self is only pinned temporarily for thread safety, then released to allow the job to be cancelled (by calling destructor)
				// if the panel is closed. For the remainer of the scope, we protect the validity of the Self pointer with NumAsyncTasks
				// counter instead (see destructor).
				SImgMediaProcessEXR* SelfPtr;
				{
					const TSharedPtr<SImgMediaProcessEXR> Self = WeakSelf.Pin();
					if (!Self)
					{
						return;
					}
					++Self->NumAsyncTasks;	// This will protect from destruction while async task is going.
					SelfPtr = Self.Get();	// Keep raw pointer for the remainer of the scope.
				}

				SelfPtr->ProcessAllImages();
				--SelfPtr->NumAsyncTasks;

				// Close notification. Must be run on the main thread.
				Async(EAsyncExecution::TaskGraphMainThread, [WeakSelf]()
				{
					if (TSharedPtr<SImgMediaProcessEXR> Self = WeakSelf.Pin())
					{
						if (Self->ConfirmNotification.IsValid())
						{
							Self->ConfirmNotification->SetEnabled(false);
							Self->ConfirmNotification->SetCompletionState((Self->bIsCancelling) ? SNotificationItem::CS_Fail : SNotificationItem::CS_Success);
							Self->ConfirmNotification->ExpireAndFadeout();
						}

						// Done with processing.
						Self->bIsProcessing = false;
						Self->bIsCancelling = false;
						Self->UpdateWidgets();
					}
				});
			});
		}
	}

	return FReply::Handled();
}

FReply SImgMediaProcessEXR::OnCancelClicked()
{
	if (bIsProcessing)
	{
		bIsCancelling = true;
		UpdateWidgets();
	}

	return FReply::Handled();
}

void SImgMediaProcessEXR::ProcessAllImages()
{
	using namespace UE::ImgMediaEditor::ProcessEXR;
	TRACE_CPUPROFILER_EVENT_SCOPE(SImgMediaProcessEXR::ProcessAllImages);

	int32 InTileWidth = Options->bEnableTiling ? Options->TileSizeX : 0;
	int32 InTileHeight = Options->bEnableTiling ? Options->TileSizeY : 0;
	int32 TileBorder = 0; // Note: virtual texture support is shelved for now.
	bool bEnableMips = Options->bEnableMipMapping;

	// Create output directory.
	FString OutPath = Options->OutputPath.Path;
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!FPaths::DirectoryExists(OutPath))
	{
		if (!PlatformFile.CreateDirectoryTree(*OutPath))
		{
			CancelWithError(FText::Format(LOCTEXT("FailedToCreateOutputTree", "Failed to create output directory \"{0}\""), FText::FromString(OutPath)));
			return;
		}
	}

	// Get source files.
	const FString InputPath = FPaths::GetPath(Options->InputPath.FilePath);
	if (InputPath.IsEmpty())
	{
		CancelWithError(FText::Format(LOCTEXT("InvalidInputPath", "Specified input path \"{0}\" is invalid."), FText::FromString(Options->InputPath.FilePath)));
		return;
	}

	const FString SequencePath = FPaths::ConvertRelativePathToFull(InputPath);
	if (!FPaths::DirectoryExists(SequencePath))
	{
		CancelWithError(FText::Format(LOCTEXT("InvalidSequencePath", "Specified input directory directory \"{0}\" does not exist."), FText::FromString(SequencePath)));
		return;
	}

	TArray<FString> FoundFiles;
	IFileManager::Get().FindFiles(FoundFiles, *SequencePath, TEXT("*"));
	FoundFiles.RemoveAll([](const FString& FileName) { return FileName.StartsWith(TEXT(".")); });
	FoundFiles.Sort();
	UE_LOGF(LogImgMediaEditor, Warning, "Found %i image files in %ls to import.", FoundFiles.Num(), *SequencePath);
	if (FoundFiles.Num() == 0)
	{
		CancelWithError(LOCTEXT("NoFilesToImport", "No files to import."));
		return;
	}
	
	// Create image wrapper
	FString Ext = FPaths::GetExtension(FoundFiles[0]);
	EImageFormat ImageFormat = ImageWrapperHelper::GetImageFormat(Ext);

	if (ImageFormat == EImageFormat::Invalid)
	{
		CancelWithError(FText::Format(LOCTEXT("InvalidFileFormat", "Invalid file format \"{0}\""), FText::FromString(Ext)));
		return;
	}
	
	// Get number of threads to use.
	int32 NumThreads = Options->NumThreads;
	if (NumThreads <= 0)
	{
		NumThreads = 8;
	}

	// Loop through all files.
	int FileIndex = 0;
	int TotalNum = FoundFiles.Num();
	std::atomic<int32> NumActive = 0;

	for (const FString& FileName : FoundFiles)
	{
		// Wait for threads to finish if we have to many.
		while (NumActive >= NumThreads)
		{
			FPlatformProcess::Sleep(0.1f);
		}
		++NumActive;

		// Update notification with current status.
		SetNotificationTextSafe(ConfirmNotification,
			FText::Format(LOCTEXT("ImgMediaCompleted", "ImgMedia Completed {0}/{1}"), FText::AsNumber(FileIndex+1), FText::AsNumber(TotalNum)));

		Async(EAsyncExecution::Thread, [this, SequencePath, FileName, OutPath,
			Ext, &NumActive, InTileWidth, InTileHeight,
			TileBorder, bEnableMips]() mutable
		{
			FString FullFileName = FPaths::Combine(SequencePath, FileName);
			FImage Image;

			// Load image into buffer.
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(SImgMediaProcessEXR::ProcessImageCustom:LoadImage);
				if (!FImageUtils::LoadImage(*FullFileName, Image))
				{
					UE_LOGF(LogImgMediaEditor, Error, "Failed to load %ls", *FullFileName);
					--NumActive;
					return;
				}
			}

			// Convert to linear float16 if not already.
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(SImgMediaProcessEXR::ProcessImageCustom:ChangeFormat);
				Image.ChangeFormat(ERawImageFormat::RGBA16F, EGammaSpace::Linear);
			}

			FImageParameters ImageParameters;
			GetImageParameters(Ext, FullFileName, ImageParameters);

			FString Name = FPaths::Combine(OutPath, FileName);
			if (Ext != TEXT("exr"))
			{
				Name = FPaths::ChangeExtension(Name, TEXT("exr"));
			}

			ProcessImageCustom(Image, InTileWidth, InTileHeight, TileBorder,
				bEnableMips, ImageParameters, Name);

			--NumActive;
		});

		++FileIndex;

		// Do we want to cancel?
		if (bIsCancelling)
		{
			break;
		}
	}

	// Wait for all our tasks to finish.
	while (NumActive > 0)
	{
		FPlatformProcess::Sleep(0.2f);
	}
}

void SImgMediaProcessEXR::GetImageParameters(const FString& Ext, const FString& File,
	FImageParameters& ImageParameters)
{
	ImageParameters.bHasAlphaChannel = true;
	ImageParameters.bIsDataWindowValid = false;
	ImageParameters.bIsDisplayWindowValid = false;

	// We just support EXR at the moment.
	if (Ext == TEXT("exr"))
	{
		FRgbaInputFile InputFile(File);
		ImageParameters.DataWindow = InputFile.GetDataWindowRect();
		ImageParameters.DisplayWindow = InputFile.GetDisplayWindow();
		ImageParameters.bHasAlphaChannel = InputFile.GetNumChannels() == 4;
		ImageParameters.bIsDataWindowValid = true;
		ImageParameters.bIsDisplayWindowValid = true;
	}
}

void SImgMediaProcessEXR::GetGlobalImageParameters()
{
	FString SequencePath = FPaths::ConvertRelativePathToFull(FPaths::GetPath(Options->InputPath.FilePath));

	TArray<FString> FoundFiles;
	IFileManager::Get().FindFiles(FoundFiles, *SequencePath, TEXT("*"));
	FoundFiles.RemoveAll([](const FString& FileName) { return FileName.StartsWith(TEXT(".")); });

	if (FoundFiles.Num() > 0)
	{
		FString Ext = FPaths::GetExtension(FoundFiles[0]);
		FString FullFileName = FPaths::Combine(SequencePath, FoundFiles[0]);

		GetImageParameters(Ext, FullFileName, GlobalImageParameters);
		GlobalImageParameters.bHasAlphaChannel = false;
	}
}

void SImgMediaProcessEXR::ProcessImageCustom(const FImage& InImage,
	int32 InTileWidth, int32 InTileHeight, int32 InTileBorder, bool bInEnableMips,
	const FImageParameters& ImageParameters, const FString& InName)
{
#if IMGMEDIAEDITOR_EXR_SUPPORTED_PLATFORM
	TRACE_CPUPROFILER_EVENT_SCOPE(SImgMediaProcessEXR::ProcessImageCustom);
	// Get image data.
	int32 Width = InImage.GetWidth();
	int32 Height = InImage.GetHeight();
	TArray64<uint8> RawData = InImage.RawData;

	ProcessImageCustomRawData(RawData, Width, Height,
		InTileWidth, InTileHeight, InTileBorder, bInEnableMips,
		ImageParameters, InName);
#else // IMGMEDIAEDITOR_EXR_SUPPORTED_PLATFORM
	UE_LOGF(LogImgMediaEditor, Error, "EXR not supported on this platform.");
#endif // IMGMEDIAEDITOR_EXR_SUPPORTED_PLATFORM
}

void SImgMediaProcessEXR::ProcessImageCustomRawData(TArray64<uint8>& RawData,
	int32 Width, int32 Height,
	int32 InTileWidth, int32 InTileHeight, int32 InTileBorder, bool bInEnableMips,
	const FImageParameters& ImageParameters, const FString& InName)
{
#if IMGMEDIAEDITOR_EXR_SUPPORTED_PLATFORM
	TRACE_CPUPROFILER_EVENT_SCOPE(SImgMediaProcessEXR::ProcessImageCustomRawData);
	int32 DestWidth = Width;
	int32 DestHeight = Height;
	int32 NumTilesX = InTileWidth > 0 ? (Width + InTileWidth - 1) / InTileWidth : 1;
	int32 NumTilesY = InTileHeight > 0 ? (Height + InTileHeight - 1) / InTileHeight : 1;
	int32 TileWidth = InTileWidth;
	int32 TileHeight = InTileHeight;
	int32 BytesPerPixel = RawData.Num() / (Width * Height);
	int32 BytesPerPixelPerChannel = 16 / 8; // <- 2 bytes per channel, assumes RGBA16F
	int32 NumChannels = BytesPerPixel / BytesPerPixelPerChannel;
	int32 DestNumChannels = NumChannels;

	// ImageWrapper always returns an alpha channel, so make sure we really have one.
	if ((DestNumChannels == 4) && ((ImageParameters.bHasAlphaChannel == false) || Options->bRemoveAlphaChannel))
	{
		// Remove the alpha channel as its not needed.
		RemoveAlphaChannel(RawData);
		NumChannels = 3;
		DestNumChannels = 3;
		BytesPerPixel = BytesPerPixelPerChannel * NumChannels;
	}

	TArray64<uint8> TileBuffer;
	TArray64<uint8> TintBuffer;
	bool bIsTiled = (NumTilesX > 1) || (NumTilesY > 1);
	if (bIsTiled)
	{
		// Take border into account.
		DestWidth = Width + InTileBorder * 2 * NumTilesX;
		DestHeight = Height + InTileBorder * 2 * NumTilesY;
	}

	uint8* RawDataPtr = RawData.GetData();

	// Names for our channels.
	const FString RChannelName = FString(TEXT("R"));
	const FString GChannelName = FString(TEXT("G"));
	const FString BChannelName = FString(TEXT("B"));
	const FString AChannelName = FString(TEXT("A"));

	FIntPoint Stride(2, 0);
	FIntRect DataWindow = ImageParameters.bIsDataWindowValid ? ImageParameters.DataWindow :
		FIntRect(0, 0, DestWidth - 1, DestHeight - 1);
	FIntRect DisplayWindow = ImageParameters.bIsDisplayWindowValid ? ImageParameters.DisplayWindow :
		FIntRect(0, 0, DestWidth - 1, DestHeight - 1);

	// Create tiled exr file.
	FTiledOutputFile OutFile(DisplayWindow.Min, DisplayWindow.Max,
		DataWindow.Min, DataWindow.Max, bIsTiled);

	// Add channels.
	if (DestNumChannels == 4)
	{
		OutFile.AddChannel(AChannelName);
	}
	if (DestNumChannels >= 3)
	{
		OutFile.AddChannel(BChannelName);
		OutFile.AddChannel(GChannelName);
		OutFile.AddChannel(RChannelName);
	}

	OutFile.CreateOutputFile(InName, TileWidth, TileHeight, bInEnableMips, 1);

	if (DestNumChannels == 4)
	{
		OutFile.AddFrameBufferChannel(AChannelName, nullptr, Stride);
	}
	if (DestNumChannels >= 3)
	{
		OutFile.AddFrameBufferChannel(BChannelName, nullptr, Stride);
		OutFile.AddFrameBufferChannel(GChannelName, nullptr, Stride);
		OutFile.AddFrameBufferChannel(RChannelName, nullptr, Stride);
	}

	// Flip between 2 buffers making mips.
	TArray64<uint8> RawData2;
	uint8* MipBuffer[2];
	MipBuffer[0] = RawDataPtr;
	MipBuffer[1] = nullptr;
	int32 CurrentMipBufferIndex = 0;

	// Loop over each mip level.
	TRACE_CPUPROFILER_EVENT_SCOPE(SImgMediaProcessEXR::ProcessImageCustom:CreateMips);
	int32 NumMips = OutFile.GetNumberOfMipLevels();
	int32 MipSourceWidth = Width;
	int32 MipSourceHeight = Height;
	int32 LastMipWidth = 0;
	int32 LastMipHeight = 0;
	for (int32 MipLevel = 0; MipLevel < NumMips; MipLevel++)
	{
		int32 MipWidth = OutFile.GetMipWidth(MipLevel);
		int32 MipHeight = OutFile.GetMipHeight(MipLevel);
		uint8* CurrentBuffer = MipBuffer[CurrentMipBufferIndex];
		uint8* LastBuffer = MipBuffer[CurrentMipBufferIndex ^ 1];

		// Allocate space for the other buffer.
		if (CurrentBuffer == nullptr)
		{
			RawData2.AddUninitialized(MipWidth* MipHeight* BytesPerPixel);
			MipBuffer[CurrentMipBufferIndex] = RawData2.GetData();
			CurrentBuffer = MipBuffer[CurrentMipBufferIndex];
		}

		// Generate mip data.
		if (MipLevel != 0)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SImgMediaProcessEXR::ProcessImageCustom:GenerateMipData);
			int32 SourceStrideX = NumChannels;
			int32 SourceStrideY = LastMipWidth * NumChannels;
			for (int32 PixelY = 0; PixelY < MipHeight; ++PixelY)
			{
				for (int32 PixelX = 0; PixelX < MipWidth; ++PixelX)
				{
					int32 PixelOffset = (PixelX + PixelY * MipWidth) * NumChannels;
					for (int32 Channel = 0; Channel < NumChannels; ++Channel)
					{
						// Box filter.
						int32 SourceWidth = LastMipWidth;
						int32 SourcePixelOffset = (PixelX + PixelY * SourceWidth) * NumChannels * 2 +
							Channel;
						FFloat16 SourcePixels[4];
						SourcePixels[0] =
							((FFloat16*)LastBuffer)[SourcePixelOffset];
						SourcePixels[1] =
							((FFloat16*)LastBuffer)[SourcePixelOffset + SourceStrideX];
						SourcePixels[2] =
							((FFloat16*)LastBuffer)[SourcePixelOffset + SourceStrideY];
						SourcePixels[3] =
							((FFloat16*)LastBuffer)[SourcePixelOffset + SourceStrideX + SourceStrideY];

						((FFloat16*)CurrentBuffer)[PixelOffset + Channel] =
							(SourcePixels[0] + SourcePixels[1] + SourcePixels[2] + SourcePixels[3]) * 0.25f;
					}
				}
			}
		}

		// Tint mip levels?
		if (Options->bEnableMipLevelTint)
		{
			TintData(CurrentBuffer, TintBuffer,
				MipLevel, MipWidth, MipHeight,
				NumChannels);
			CurrentBuffer = TintBuffer.GetData();
		}

		// Write to EXR.
		TRACE_CPUPROFILER_EVENT_SCOPE(SImgMediaProcessEXR::ProcessImageCustom:WriteEXR);
		
		{
			Stride.X = BytesPerPixel;
			Stride.Y = MipWidth * BytesPerPixel;
			int64 BufferOffset = -ImageParameters.DataWindow.Min.X * BytesPerPixel -
				ImageParameters.DataWindow.Min.Y * BytesPerPixel * MipWidth;
			int64 SingleBufferOffset = BytesPerPixelPerChannel;
			

			OutFile.UpdateFrameBufferChannel(RChannelName, CurrentBuffer + BufferOffset, Stride);
			BufferOffset += SingleBufferOffset;
			OutFile.UpdateFrameBufferChannel(GChannelName, CurrentBuffer + BufferOffset, Stride);
			BufferOffset += SingleBufferOffset;
			OutFile.UpdateFrameBufferChannel(BChannelName, CurrentBuffer + BufferOffset, Stride);
			BufferOffset += SingleBufferOffset;

			if (DestNumChannels == 4)
			{
				OutFile.UpdateFrameBufferChannel(AChannelName, CurrentBuffer + BufferOffset, Stride);
			}
		}

		OutFile.SetFrameBuffer();

		{
			int32 X2 = OutFile.GetNumXTiles(MipLevel) - 1;
			X2 = FMath::Max(0, X2);
			int32 Y2 = OutFile.GetNumYTiles(MipLevel) - 1;
			Y2 = FMath::Max(0, Y2);
			OutFile.WriteTiles(0, X2, 0, Y2, MipLevel);
		}

		// Switch buffers.
		CurrentMipBufferIndex ^= 1;
		MipSourceHeight /= 2;
		MipSourceWidth /= 2;
		LastMipWidth = MipWidth;
		LastMipHeight = MipHeight;
	}

#else // IMGMEDIAEDITOR_EXR_SUPPORTED_PLATFORM
	UE_LOGF(LogImgMediaEditor, Error, "EXR not supported on this platform.");
#endif // IMGMEDIAEDITOR_EXR_SUPPORTED_PLATFORM
}

void SImgMediaProcessEXR::RemoveAlphaChannel(TArray64<uint8>& Buffer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SImgMediaProcessEXR::RemoveAlphaChannel);

	int32 BytesPerPixelPerChannel = 2;
	int64 BufferSize = Buffer.Num() / BytesPerPixelPerChannel;
	uint16* BufferPtr = (uint16*)(Buffer.GetData());
	
	// Loop through the buffer.
	int64 OutIndex = 0;
	for (int64 Index = 0; Index < BufferSize; ++Index)
	{
		// Skip every fourth channel (i.e. the alpha channel).
		if ((Index & 0x3) != 3)
		{
			// Copy the data in place.
			BufferPtr[OutIndex] = BufferPtr[Index];
			OutIndex++;
		}
	}

	// Don't bother shrinking as its just a waste and extra work.
	Buffer.SetNum((BufferSize * 3) / 4, EAllowShrinking::No);
}

void SImgMediaProcessEXR::TintData(uint8* SourceData, TArray64<uint8>& DestArray,
	int32 InMipLevel, int32 InWidth, int32 InHeight, int32 InNumChannels)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SImgMediaProcessEXR::TintData);

	// Get tint colour.
	FLinearColor TintColor = FLinearColor::White;
	if (Options->MipLevelTints.Num() > 0)
	{
		TintColor = Options->MipLevelTints[InMipLevel % Options->MipLevelTints.Num()];
	}

	// Set up destination buffer.
	DestArray.Reset();
	DestArray.AddUninitialized(((int64)InWidth) * ((int64)InHeight) * InNumChannels * 2);
	FFloat16* Buffer = (FFloat16*)SourceData;
	FFloat16* OutBuffer = (FFloat16*)DestArray.GetData();
		
	// Tint buffer.
	for (int32 PixelY = 0; PixelY < InHeight; ++PixelY)
	{
		for (int32 PixelX = 0; PixelX < InWidth; ++PixelX)
		{
			OutBuffer[0] = (Buffer[0].GetFloat() + TintColor.R) * 0.5f;
			OutBuffer[1] = (Buffer[1].GetFloat() + TintColor.G) * 0.5f;
			OutBuffer[2] = (Buffer[2].GetFloat() + TintColor.B) * 0.5f;
			if (InNumChannels == 4)
			{
				OutBuffer[3] = Buffer[3];
			}
				
			Buffer += InNumChannels;
			OutBuffer += InNumChannels;
		}
	}
}

void SImgMediaProcessEXR::TileData(uint8* SourceData, TArray64<uint8>& DestArray,
	int32 SourceWidth, int32 SourceHeight, int32 DestWidth, int32 DestHeight,
	int32 NumTilesX, int32 NumTilesY,
	int32 TileWidth, int32 TileHeight, int32 InTileBorder,
	int32 BytesPerPixel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SImgMediaProcessEXR::TileData);

	// We don't support tile borders larger than a tile size,
	// but this shuld not happen in practice.
	if ((InTileBorder > TileWidth) || (InTileBorder > TileHeight))
	{
		UE_LOGF(LogImgMediaEditor, Error, "Tile border is larger than tile size. Clamping to tile size.");
		InTileBorder = FMath::Min(TileWidth, TileHeight);
	}

	// Set up destination buffer.
	DestArray.Reset();
	DestArray.AddUninitialized(DestWidth * DestHeight * BytesPerPixel);

	uint8* DestData = DestArray.GetData();
	int32 DestTileWidth = TileWidth + InTileBorder * 2;
	int32 DestTileHeight = TileHeight + InTileBorder * 2;

	// Make sure our output tile size is not bigger than the output size.
	if ((DestTileWidth > DestWidth) || (DestTileHeight > DestHeight))
	{
		// This is not a valid mip level, so just ignore.
		return;
	}

	int32 BytesPerTile = TileWidth * TileHeight * BytesPerPixel;
	int32 ByterPerDestTile = DestTileWidth * DestTileHeight * BytesPerPixel;
	uint8* DestTile = DestData;

	// Loop over y tiles.
	for (int32 TileY = 0; TileY < NumTilesY; ++TileY)
	{
		// Loop over x tiles.
		for (int32 TileX = 0; TileX < NumTilesX; ++TileX)
		{
			// Get address of the source and destination tiles.
			uint8* SourceTile = SourceData +
				(TileX * TileWidth + TileY * SourceWidth * TileHeight) * BytesPerPixel;
			
			// If this tile is over the right edge of our image, then make this tile smaller
			// so it does not exceed the image size.
			int32 NumberOfPixelsToCopy = TileWidth;
			int32 ThisDestTileWidth = DestTileWidth;
			if ((TileX + 1) * TileWidth > SourceWidth)
			{
				NumberOfPixelsToCopy = SourceWidth - TileX * TileWidth;
				ThisDestTileWidth -= TileWidth - NumberOfPixelsToCopy;
			}

			// Create a left border.
			int32 DestTileOffset = 0;
			if (TileX > 0)
			{
				NumberOfPixelsToCopy += InTileBorder;
				// Offset the source to get the extra pixels.
				SourceTile -= InTileBorder * BytesPerPixel;
			}
			else
			{
				// Offset the destination as we are skipping this border as we have no data.
				DestTileOffset = InTileBorder * BytesPerPixel;
				DestTile += DestTileOffset;
			}

			// Create a right border.
			if (TileX < NumTilesX - 1)
			{
				NumberOfPixelsToCopy += InTileBorder;
			}

			// If this tile is over the bottom edge of our image, then make this tile smaller
			// so it does not exceed the image size.
			int32 ThisDestTileHeight = DestTileHeight;
			if ((TileY + 1) * TileHeight > SourceHeight)
			{
				ThisDestTileHeight = SourceHeight - TileY * TileHeight;
			}

			// Loop over each row in the tile.
			for (int32 Row = 0; Row < ThisDestTileHeight; ++Row)
			{
				// Make sure we don't go beyond the source data.
				int32 SourceRow = Row - InTileBorder;
				if (TileY == 0)
				{
					SourceRow = FMath::Max(SourceRow, 0);
				}
				if (TileY == NumTilesY - 1)
				{
					SourceRow = FMath::Min(SourceRow, TileHeight - 1);
				}

				uint8* SourceLine = SourceTile + SourceRow * SourceWidth * BytesPerPixel;
				uint8* DestLine = DestTile;

				// Copy the main data.
				FMemory::Memcpy(DestLine, SourceLine, NumberOfPixelsToCopy * BytesPerPixel);

				// Increment our pointer to the next tile.
				// We have to remove any DestTileOffset we applied earlier.
				DestTile += ThisDestTileWidth * BytesPerPixel - DestTileOffset;
			}
		}
	}
}

void SImgMediaProcessEXR::HandleProcessing()
{
	using namespace UE::ImgMediaEditor::ProcessEXR;

	// Are we processing?
	if (bIsProcessing)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SImgMediaProcessEXR::HandleProcessing);
		// We did not cancel yet?
		bool bShouldExit = false;
		if ((MediaPlayer != nullptr) && (bIsCancelling == false))
		{
			// Get which frame the player is on.
			int32 PlayerFrame = 0;
			if (FrameDuration.GetTotalSeconds() > 0.0f)
			{
				PlayerFrame = (int32)FMath::RoundHalfToEven(MediaPlayer->GetTime().GetTotalSeconds() / FrameDuration.GetTotalSeconds());
			}

			UE_LOGF(LogImgMediaEditor, Verbose,
				"ProcessImages Time:%f PlayerTime:%f Duration:%f Frame:%d",
				CurrentTime.GetTotalSeconds(),
				MediaPlayer->GetTime().GetTotalSeconds(),
				MediaPlayer->GetDuration().GetTotalSeconds(), PlayerFrame);

			// Has the player stopped playing?
			if (MediaPlayer->IsClosed() || MediaPlayer->HasError())
			{
				CancelWithError(LOCTEXT("ErrorPlayerStopped", "Player has stopped playing"));
				bShouldExit = true;
			}
			// Watchdog: have we made no progress for too long?
			else if (const float ProgressTimeoutSeconds = CVarProgressTimeoutSeconds.GetValueOnGameThread();
				(ProgressTimeoutSeconds > 0.0f) && ((FPlatformTime::Seconds() - LastProgressTimeSeconds) > ProgressTimeoutSeconds))
			{
				CancelWithError(FText::Format(
					LOCTEXT("ErrorPlayerStalled", "Media player did not produce a frame for {0} seconds (frame {1} of {2}). Aborting."),
					FText::AsNumber((int32)ProgressTimeoutSeconds),
					FText::AsNumber(CurrentFrameIndex),
					FText::AsNumber((int32)(MediaPlayer->GetDuration().GetTotalSeconds() / FMath::Max(FrameDuration.GetTotalSeconds(), KINDA_SMALL_NUMBER)))));
				bShouldExit = true;
			}
			// Is this the frame we want?
			else if ((MediaPlayer->IsPreparing() == false) && (CurrentFrameIndex == PlayerFrame))
			{
				// Are we set up yet?
				if (RenderTarget == nullptr)
				{
					CreateRenderTarget();
					// Get frame duration.
					float FrameRate = MediaPlayer->GetVideoTrackFrameRate(INDEX_NONE, INDEX_NONE);
					if (FrameRate <= 0.0f)
					{
						FrameRate = 24.0f;
					}
					FrameDuration = FTimespan::FromSeconds(1.0f / FrameRate);
				}

				// Did we get a render target?
				if (RenderTarget != nullptr)
				{
					// Copy media texture to our render target.
					DrawTextureToRenderTarget();

					// Process this render.
					FImage Image;
					bool bReadSuccess = FImageUtils::GetRenderTargetImage(RenderTarget, Image);

					if (bReadSuccess)
					{
						Image.ChangeFormat(ERawImageFormat::RGBA16F,EGammaSpace::Linear);
						// "Image" is not used in following code, just Image.RawData as uint8
						//	this file is hard-coded to assume 16F pixels

						int32 Width = RenderTarget->GetSurfaceWidth();
						int32 Height = RenderTarget->GetSurfaceHeight();
						int32 InTileWidth = Options->bEnableTiling ? Options->TileSizeX : 0;
						int32 InTileHeight = Options->bEnableTiling ? Options->TileSizeY : 0;
						int32 TileBorder = 0; // Note: virtual texture support is shelved for now.
						bool bEnableMips = Options->bEnableMipMapping;
						
						FString OutPath = Options->OutputPath.Path;
						FString FileName = FString::Printf(TEXT("image%05d.exr"), CurrentFrameIndex);
						FString Name = FPaths::Combine(OutPath, FileName);

						Async(EAsyncExecution::Thread, [this, RawData = MoveTemp(Image.RawData), Width, Height,
							InTileWidth, InTileHeight, TileBorder, bEnableMips, Name]() mutable
						{
							++NumAsyncTasks;
							ProcessImageCustomRawData(RawData, Width, Height,
								InTileWidth, InTileHeight, TileBorder, bEnableMips,
								GlobalImageParameters, Name);
							--NumAsyncTasks;
						});
					}
					else
					{
						UE_LOGF(LogImgMediaEditor, Error, "ProcessImages failed to get raw data.");
					}

					// Update notification.
					if (ConfirmNotification.IsValid())
					{
						ConfirmNotification->SetText(
							FText::Format(LOCTEXT("ImgMediaCompleted2", "ImgMedia Completed {0}"),
								FText::AsNumber(CurrentFrameIndex)));
					}

					// Next frame.
					CurrentTime += FrameDuration;
					CurrentFrameIndex++;
					LastProgressTimeSeconds = FPlatformTime::Seconds();
					if (CurrentTime >= MediaPlayer->GetDuration())
					{
						bShouldExit = true;
					}
					else
					{
						MediaPlayer->SetBlockOnTimeRange(TRange<FTimespan>(CurrentTime, CurrentTime + FrameDuration));
					}
				}
			}
		}
		else
		{
			bShouldExit = true;
		}

		// Are we done?
		if (bShouldExit)
		{
			// Remove notification.
			if (ConfirmNotification.IsValid())
			{
				ConfirmNotification->SetEnabled(false);
				ConfirmNotification->SetCompletionState(bIsCancelling ? SNotificationItem::CS_Fail : SNotificationItem::CS_Success);
				ConfirmNotification->ExpireAndFadeout();
				ConfirmNotification.Reset();
			}

			// Clean up.
			bIsCancelling = false;
			bIsProcessing = false;
			CleanUp();
			UpdateWidgets();
		}
	}
}

void SImgMediaProcessEXR::HandleMediaPlayerEvent(EMediaEvent InEvent)
{
	using namespace UE::ImgMediaEditor::ProcessEXR;

	const FString EventName = MediaUtils::EventToString(InEvent);
	UE_LOGF(LogImgMediaEditor, Verbose, "ProcessEXR: Media event received: %ls", *EventName);

	switch (InEvent)
	{
	case EMediaEvent::MediaOpenFailed:
		CancelWithError(FText::Format(
			LOCTEXT("MediaOpenFailed", "Failed to open media source \"{0}\". Check that the path exists and the format is supported."),
			FText::FromString(Options.IsValid() ? Options->InputPath.FilePath : FString())));
		break;

	case EMediaEvent::MediaClosed:
		// Only treat as an error if we were actively processing (not the normal end-of-run close).
		if (bIsProcessing && !bIsCancelling)
		{
			CancelWithError(LOCTEXT("MediaClosedUnexpectedly", "Media source was closed unexpectedly during processing."));
		}
		break;

	default:
		break;
	}
}

void SImgMediaProcessEXR::CancelWithError(const FText& InMessage)
{
	using namespace UE::ImgMediaEditor::ProcessEXR;

	UE_LOGF(LogImgMediaEditor, Error, "ProcessEXR: %ls", *InMessage.ToString());
	SetNotificationTextSafe(ConfirmNotification, InMessage);
	bIsCancelling = true;
}

void SImgMediaProcessEXR::CreateRenderTarget()
{
	FIntPoint VideoDim = MediaPlayer->GetVideoTrackDimensions(INDEX_NONE, INDEX_NONE);
	if ((VideoDim.X <= 0) || (VideoDim.Y <= 0))
	{
		CancelWithError(FText::Format(
			LOCTEXT("InvalidVideoDimensions", "Media player reported invalid video dimensions ({0} x {1}). The media may not have a video track or the player has not finished preparing."),
			FText::AsNumber(VideoDim.X), FText::AsNumber(VideoDim.Y)));
		return;
	}

	RenderTarget = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), TEXT("ImgMediaProcessImages"));
	RenderTarget->RenderTargetFormat = RTF_RGBA16f;
	RenderTarget->InitAutoFormat(VideoDim.X, VideoDim.Y);
	RenderTarget->AddToRoot();
	RenderTarget->UpdateResourceImmediate(true);
}

void SImgMediaProcessEXR::DrawTextureToRenderTarget()
{
	UWorld* World = nullptr;
	World = GEditor->GetEditorWorldContext().World();
	World->FlushDeferredParameterCollectionInstanceUpdates();

	FTextureRenderTargetResource* RenderTargetResource = RenderTarget->GameThread_GetRenderTargetResource();

	UCanvas* Canvas = World->GetCanvasForDrawMaterialToRenderTarget();
	FCanvas RenderCanvas(
		RenderTargetResource,
		nullptr,
		World,
		World->GetFeatureLevel());
	Canvas->Init(RenderTarget->SizeX, RenderTarget->SizeY, nullptr, &RenderCanvas);
	Canvas->Update();

	{
		ENQUEUE_RENDER_COMMAND(FlushDeferredResourceUpdateCommand)(
			[RenderTargetResource](FRHICommandListImmediate& RHICmdList)
		{
			RenderTargetResource->FlushDeferredResourceUpdate(RHICmdList);
		});

		Canvas->K2_DrawTexture(MediaTexture, FVector2D(0, 0), FVector2D(RenderTarget->SizeX, RenderTarget->SizeY), FVector2D(0, 0));

		RenderCanvas.Flush_GameThread();
		Canvas->Canvas = nullptr;
		RenderTarget->UpdateResourceImmediate(false);
	}
}

void SImgMediaProcessEXR::CleanUp()
{
	if (MediaPlayer != nullptr)
	{
		if (MediaEventHandle.IsValid())
		{
			MediaPlayer->OnMediaEvent().Remove(MediaEventHandle);
			MediaEventHandle.Reset();
		}
		MediaPlayer->Close();
		MediaPlayer->RemoveFromRoot();
		MediaPlayer = nullptr;
	}
	if (MediaTexture != nullptr)
	{
		MediaTexture->RemoveFromRoot();
		MediaTexture = nullptr;
	}
	if (MediaSource != nullptr)
	{
		MediaSource->RemoveFromRoot();
		MediaSource = nullptr;
	}
	if (RenderTarget != nullptr)
	{
		RenderTarget->RemoveFromRoot();
		RenderTarget = nullptr;
	}
}

#undef LOCTEXT_NAMESPACE
