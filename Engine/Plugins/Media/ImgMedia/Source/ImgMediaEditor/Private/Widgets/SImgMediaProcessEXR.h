// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ImgMediaProcessEXROptions.h"
#include "Input/Reply.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/SCompoundWidget.h"

enum class EMediaEvent;

class IDetailsView;
struct FImage;
class SButton;
class SNotificationItem;
class UMediaPlayer;
class UMediaSource;
class UMediaTexture;
class UTextureRenderTarget2D;

/**
 * SImgMediaProcessEXR provides processing of image sequences.
 */
class SImgMediaProcessEXR : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SImgMediaProcessEXR){}
	SLATE_END_ARGS()

	virtual ~SImgMediaProcessEXR();

	void Construct(const FArguments& InArgs);

	/**
	 * Call this to set what the input path should be.
	 */
	void SetInputPath(const FString& Path);

	/** SWidget interface */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);


private:
	/** Contains info not contained in the image data. */
	struct FImageParameters
	{
		/** This window is inclusive, so (0, 0, 0, 0) is a 1 pixel rectangle at 0, 0. */
		FIntRect DataWindow;
		/** This window is inclusive, so(0, 0, 0, 0) is a 1 pixel rectangle at 0, 0. */
		FIntRect DisplayWindow;
		/** True if the image has an alpha channel. */
		bool bHasAlphaChannel = false;
		/** True if DataWindow can be used. */
		bool bIsDataWindowValid = false;
		/** True if DisplayWindow can be used. */
		bool bIsDisplayWindowValid = false;
	};
	/** Contains parameters to be applied to all images. */
	FImageParameters GlobalImageParameters;

	/**
	 * Updates our widgets based on the current state.
	 */
	void UpdateWidgets();
	
	/** Called when we click on the process images button. */
	FReply OnProcessImagesClicked();
	/** Called when we click on the cancel button. */
	FReply OnCancelClicked();

	/**
	 * Processes all images in the sequence and generate tiles/mips.
	 */
	void ProcessAllImages();
	
	/**
	  * Gets information on the file, e.g does it have an alpha channel.
	  *
	  * @param Ext				File extension.
	  * @param File				Full path to file.
	  * @param ImageParameters	Will be filled in with information from the file.
	  */
	void GetImageParameters(const FString& Ext, const FString& File, FImageParameters& ImageParameters);

	/**
	 * Gets parameters that can be applied to all files.
	 */
	void GetGlobalImageParameters();

	/**
	 * Processess a single image and writes out a file.
	 * Tiles and mips may be generated.
	 * This does NOT run on the game thread.
	 *
	 * @param InImage			FImage of the source image.
	 * @param InTileWidth		Desired width of tiles.
	 * @param InTileHeight		Desired height of tiles.
	 * @param InTileBorder		Number of pixels to duplicate along a tile edge.
	 * @param bInEnableMips		Turn on mip mapping.
	 * @param ImageParameters	Contains information on the image.
	 * @param InName			Full path and name of file to write.
	 */
	void ProcessImageCustom(const FImage& InImage,
		int32 InTileWidth, int32 InTileHeight, int32 InTileBorder, 
		bool bInEnableMips, const FImageParameters& ImageParameters, const FString& InName);

	/**
	 * Processess a single image and writes out a file.
	 * Tiles and mips may be generated.
	 * This might not run on the game thread.
	 *
	 * @param RawData			Image pixel data (RGBA FFloat16).
	 * @param Width				Width of image.
	 * @param Height			Height of image.
	 * @param InTileWidth		Desired width of tiles.
	 * @param InTileHeight		Desired height of tiles.
	 * @param InTileBorder		Number of pixels to duplicate along a tile edge.
	 * @param bInEnableMips		Turn on mip mapping.
	 * @param ImageParameters	Contains information on the image.
	 * @param InName			Full path and name of file to write.
	 */
	void ProcessImageCustomRawData(TArray64<uint8>& RawData,
		int32 Width, int32 Height,
		int32 InTileWidth, int32 InTileHeight, int32 InTileBorder, bool bInEnableMips,
		const FImageParameters& ImageParameters, const FString& InName);

	/**
	 * Removes the alpha channel from a buffer.
	 * This will be done in place.
	 * Assumes 2 bytes per channel, 4 channels, alpha is the last channel.
	 * 
	 * @param Buffer			Buffer to remove alpha from.
	 */
	void RemoveAlphaChannel(TArray64<uint8>& Buffer);

	/**
	 * Tints a source buffer and outputs it to a destination.
	 *
	 * @param SourceData		Source image.
	 * @param DestArray			Destination image.
	 * @param InMipLevel		The mip level of this image.
	 * @param InWidth			Width of source in pixels.
	 * @param InHeight			Height of source in pixels.
	 * @param InNumChannels		Number of channels in the image.
	 */
	void TintData(uint8* SourceData, TArray64<uint8>& DestArray, 
		int32 InMipLevel, int32 InWidth, int32 InHeight,
		int32 InNumChannels);

	/**
	 * Creates tiles from a source and outputs it to a destination.
	 *
	 * @param SourceData		Source image.
	 * @param DestArray			Destination image.
	 * @param SourceWidth		Width of source in pixels.
	 * @param SourceHeight		Height of source in pixels.
	 * @param DestWidth			Width of destination in pixels.
	 * @param DestHeight		Height of destination in pixels.
	 * @param NumTilesX			Number of tiles in X direction.
	 * @param NumTilesY			Number of tiles in Y direction.
	 * @param TileWidth			Width of a tile (without borders) in pixels.
	 * @param TileHeight		Height of a tile (without borders) in pixels.
	 * @param InTileBorder		Size of border in pixels.
	 * @param BytesPerPixel		Number of bytes per pixel.
	 */
	void TileData(uint8* SourceData, TArray64<uint8>& DestArray,
		int32 SourceWidth, int32 SourceHeight, int32 DestWidth, int32 DestHeight,
		int32 NumTilesX, int32 NumTilesY,
		int32 TileWidth, int32 TileHeight, int32 InTileBorder,
		int32 BytesPerPixel);

	/**
	 * Call this every frame to handle processing.
	 */
	void HandleProcessing();

	/**
	 * Handles media events from the player (e.g. MediaOpenFailed).
	 *
	 * @param InEvent The media event received from the player.
	 */
	void HandleMediaPlayerEvent(EMediaEvent InEvent);

	/**
	 * Cancels processing and reports an error in the notification and log.
	 *
	 * @param InMessage Human-readable error message.
	 */
	void CancelWithError(const FText& InMessage);

	/**
	 * Creates a render target.
	 */
	void CreateRenderTarget();

	/**
	 * Draws the media texture to our render target.
	 */
	void DrawTextureToRenderTarget();

	/**
	 * Clean up after processing is done.
	 */
	void CleanUp();

	/** Holds our details view. */
	TSharedPtr<class IDetailsView> DetailsView;
	/** Object that holds our options. */
	TStrongObjectPtr<UImgMediaProcessEXROptions> Options;
	/** Holds the button to start processing. */
	TSharedPtr<SButton> StartButton;
	/** Holds the button to cancel processing. */
	TSharedPtr<SButton> CancelButton;
	/** Notification to update user on our progress. */
	TSharedPtr<SNotificationItem> ConfirmNotification;

	/** Used to read in media. */
	TObjectPtr<UMediaPlayer> MediaPlayer;
	/** Handle for OnMediaEvent subscription on MediaPlayer. */
	FDelegateHandle MediaEventHandle;
	/** Texture for the player. */
	TObjectPtr<UMediaTexture> MediaTexture;
	/** Source for the player. */
	TObjectPtr<UMediaSource> MediaSource;
	/** Render target to render the media texture to. */
	TObjectPtr<UTextureRenderTarget2D> RenderTarget;
	/** The current time we are processing. */
	FTimespan CurrentTime;
	/** How long a frame is. */
	FTimespan FrameDuration;
	/** Current frame index we are processing. */
	int32 CurrentFrameIndex = 0;
	/** Wall-clock time of the last forward progress (frame processed or player ready). Used to time out stuck players. */
	double LastProgressTimeSeconds = 0.0;

	/** True if we are currently processing. */
	bool bIsProcessing = false;
	/** True if we want to cancel processing. */
	bool bIsCancelling = false;
	/** True if we are using a player to read in the images. */
	bool bUsePlayer = false;

	/**
	 * Counter for the number of "async tasks" that are currently running.
	 * This used by the main thread to protect against the widget being deleted while in use by the async tasks.
	 */
	std::atomic<int32> NumAsyncTasks = 0;
};
