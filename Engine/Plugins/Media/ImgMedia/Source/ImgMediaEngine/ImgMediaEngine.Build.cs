// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ImgMediaEngine : ModuleRules
	{
		public ImgMediaEngine(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"MediaAssets",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
				});

			// Are we using the engine?
			if (Target.bCompileAgainstEngine)
			{
				PublicDependencyModuleNames.AddRange(
				new string[] {
					"CoreUObject",
					"Engine",
				});
			}

			// Are we using the editor?
			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
				new string[] {
					"UnrealEd",
				});
			}

		}
	}
}
