// Copyright Epic Games, Inc. All Rights Reserved.


#include "ImgMediaEngineModule.h"

void FImgMediaEngineModule::StartupModule()
{
}

void FImgMediaEngineModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FImgMediaEngineModule, ImgMediaEngine);
