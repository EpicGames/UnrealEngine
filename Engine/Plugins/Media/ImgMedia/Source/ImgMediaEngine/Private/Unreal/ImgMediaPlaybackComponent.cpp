// Copyright Epic Games, Inc. All Rights Reserved.

#include "Unreal/ImgMediaPlaybackComponent.h"

#include "Components/PrimitiveComponent.h"
#include "Materials/MaterialInterface.h"
#include "MediaTexture.h"
#include "MediaTextureTracker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImgMediaPlaybackComponent)

UDEPRECATED_ImgMediaPlaybackComponent::UDEPRECATED_ImgMediaPlaybackComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAutoActivate = true;
}

void UDEPRECATED_ImgMediaPlaybackComponent::BeginPlay()
{
	Super::BeginPlay();

	RegisterWithMipMapInfo();
}

void UDEPRECATED_ImgMediaPlaybackComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	UnregisterWithMipMapInfo();

	Super::EndPlay(EndPlayReason);
}

void UDEPRECATED_ImgMediaPlaybackComponent::RegisterWithMipMapInfo()
{
	// Get all texures used by our actor.
	AActor* Owner = GetOwner();
	if (Owner != nullptr)
	{
		// Create info.
		FMediaTextureTracker& MediaTextureTracker = FMediaTextureTracker::Get();
		ObjectInfo = MakeShared<FMediaTextureTrackerObject, ESPMode::ThreadSafe>();
		ObjectInfo->Object = Owner;
		ObjectInfo->MipMapLODBias = LODBias;

		// Loop over all primitive components.
		TArray<UActorComponent*> Components;
		Owner->GetComponents(UPrimitiveComponent::StaticClass(), Components);
		for (UActorComponent* Component : Components)
		{
			UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component);
			if (PrimitiveComponent != nullptr)
			{
				// Get materials from component.
				TArray<UMaterialInterface*> Materials;
				PrimitiveComponent->GetUsedMaterials(Materials);
				for (UMaterialInterface* Material : Materials)
				{
					// Get textures from material.
					TArray<UTexture*> Textures;
					Material->GetUsedTextures(Textures);
					for (UTexture* Texture : Textures)
					{
						// Is this a media texture?
						UMediaTexture* MediaTexture = Cast<UMediaTexture>(Texture);
						if (MediaTexture != nullptr)
						{
							MediaTextures.Add(MediaTexture);
							MediaTextureTracker.RegisterTexture(ObjectInfo, MediaTexture);
						}
					}
				}
			}
		}
	}
}

void UDEPRECATED_ImgMediaPlaybackComponent::UnregisterWithMipMapInfo()
{
	if (ObjectInfo != nullptr)
	{
		FMediaTextureTracker& MediaTextureTracker = FMediaTextureTracker::Get();

		// Loop over all our media textures.
		for (TWeakObjectPtr<UMediaTexture> MediaTexturePtr : MediaTextures)
		{
			UMediaTexture* MediaTexture = MediaTexturePtr.Get();
			if (MediaTexture != nullptr)
			{
				MediaTextureTracker.UnregisterTexture(ObjectInfo, MediaTexture);
			}
		}
	}
}

