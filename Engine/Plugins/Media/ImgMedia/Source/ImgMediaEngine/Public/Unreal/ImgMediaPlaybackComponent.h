// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "ImgMediaPlaybackComponent.generated.h"

class UMediaTexture;
struct FMediaTextureTrackerObject;

/**
 * Component to help with ImgMedia playback.
 * If desired, add this to an object that displays an ImgMedia sequence.
 * Not necessary to do this, but if you do then you can get additional functionality
 * such as selective mipmap loading.
 */
UCLASS(Deprecated)
class UE_DEPRECATED(5.3, "ImgMediaPlaybackComponent functionality is deprecated. Please update your project to use the features of the MediaPlate instead.")  IMGMEDIAENGINE_API UDEPRECATED_ImgMediaPlaybackComponent : public UActorComponent
{
	GENERATED_BODY()
public:
	UDEPRECATED_ImgMediaPlaybackComponent(const FObjectInitializer& ObjectInitializer);

	/** This will be added to the calculated mipmap level. E.g. if set to 2, and you would normally be at mipmap level 1, then you will actually be at level 3. */
	UPROPERTY(EditAnywhere, Category = MipMaps)
	float LODBias = 0.0f;

	//~ UActorComponent interface
	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

protected:
	/**
	 * Registers us with the mipmap info system.
	 */
	void RegisterWithMipMapInfo();

	/**
	 * Unregisters us from the mipmap info system.
	 */
	void UnregisterWithMipMapInfo();

	/** List of media textures that are used by our object. */
	TArray<TWeakObjectPtr<UMediaTexture>> MediaTextures;

	/** Info representing this object. */
	TSharedPtr<FMediaTextureTrackerObject, ESPMode::ThreadSafe> ObjectInfo;
};
