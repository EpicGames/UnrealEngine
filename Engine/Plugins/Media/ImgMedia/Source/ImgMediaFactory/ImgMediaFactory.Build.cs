// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ImgMediaFactory : ModuleRules
	{
		public ImgMediaFactory(ReadOnlyTargetRules Target) : base(Target)
		{
			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Media",
					"ImgMedia",
					"TmvMedia"
				});

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"TimeManagement",
				});

			if (Target.Type == TargetRules.TargetType.Editor)
			{
				DynamicallyLoadedModuleNames.Add("Settings");
				PrivateIncludePathModuleNames.Add("Settings");
			}

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				DynamicallyLoadedModuleNames.Add("ImgMedia");
				DynamicallyLoadedModuleNames.Add("TmvMedia");
			}
		}
	}
}
