// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImgMediaFactoryPrivate.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Decoder/ITmvMediaDemuxerFactory.h"
#include "IImgMediaModule.h"
#include "IMediaModule.h"
#include "IMediaOptions.h"
#include "IMediaPlayerFactory.h"
#include "ITmvMediaModule.h"
#include "ImgMediaSettings.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Paths.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "UObject/NameTypes.h"

#if WITH_EDITOR
	#include "ISettingsModule.h"
	#include "ISettingsSection.h"
	#include "UObject/Class.h"
	#include "UObject/WeakObjectPtr.h"
#endif

DEFINE_LOG_CATEGORY(LogImgMediaFactory);

#define LOCTEXT_NAMESPACE "FImgMediaFactoryModule"


/**
 * Implements the ImgMediaFactory module.
 */
class FImgMediaFactoryModule
	: public IMediaPlayerFactory
	, public IModuleInterface
{
public:

	/** Default constructor. */
	FImgMediaFactoryModule()
	{ }

public:

	//~ IMediaPlayerInfo interface

	virtual bool CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		return GetPlayabilityConfidenceScore(Url, Options, OutWarnings, OutErrors) > 0 ? true : false;
	}

	virtual int32 GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		FString Scheme;
		FString Location;

		// check scheme
		if (!Url.Split(TEXT("://"), &Scheme, &Location, ESearchCase::CaseSensitive))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(LOCTEXT("NoSchemeFound", "No URI scheme found"));
			}

			return 0;
		}

		// Check for supported container files via "file" scheme (e.g. file://path/to/video.tmv).
		if (Scheme == TEXT("file"))
		{
			if (const ITmvMediaModule* TmvModule = FModuleManager::LoadModulePtr<ITmvMediaModule>("TmvMedia"))
			{
				const FString Extension = FPaths::GetExtension(Location);
				if (TmvModule->FindDemuxerFactoryForExtension(Extension).IsValid())
				{
					return 100;
				}
			}
		}

		if (!SupportedUriSchemes.Contains(Scheme))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(FText::Format(LOCTEXT("SchemeNotSupported", "The URI scheme '{0}' is not supported"), FText::FromString(Scheme)));
			}

			return 0;
		}

		return 100;
	}

	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
		auto ImgMediaModule = FModuleManager::LoadModulePtr<IImgMediaModule>("ImgMedia");
		return (ImgMediaModule != nullptr) ? ImgMediaModule->CreatePlayer(EventSink) : nullptr;
	}

	virtual FText GetDisplayName() const override
	{
		return LOCTEXT("MediaPlayerDisplayName", "Image Sequence");
	}

	virtual FName GetPlayerName() const override
	{
		static FName PlayerName(TEXT("ImgMedia"));
		return PlayerName;
	}

	virtual FGuid GetPlayerPluginGUID() const override
	{
		static FGuid PlayerPluginGUID(0x0e4a60c0, 0x2c5947ea, 0xb233562a, 0x57e5761c);
		return PlayerPluginGUID;
	}

	virtual const TArray<FString>& GetSupportedPlatforms() const override
	{
		return SupportedPlatforms;
	}

	virtual bool SupportsFeature(EMediaFeature Feature) const override
	{
		return ((Feature == EMediaFeature::VideoSamples) ||
				(Feature == EMediaFeature::VideoTracks));
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		// supported platforms
		SupportedPlatforms.Add(TEXT("Linux"));
		SupportedPlatforms.Add(TEXT("Mac"));
		SupportedPlatforms.Add(TEXT("Windows"));
		AddSupportedPlatform(FGuid(0x5636fbc1, 0xd2b54f62, 0xac8e7d4f, 0xb184b45a));

		// supported schemes
		SupportedUriSchemes.Add(TEXT("img"));

#if WITH_EDITOR
		// register settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			ISettingsSectionPtr SettingsSection = SettingsModule->RegisterSettings("Project", "Plugins", "ImgMedia",
				LOCTEXT("ImgMediaSettingsName", "IMG Media"),
				LOCTEXT("ImgMediaSettingsDescription", "Configure the IMG Media plug-in."),
				GetMutableDefault<UImgMediaSettings>()
			);
		}
#endif //WITH_EDITOR

		// register player factory
		auto MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->RegisterPlayerFactory(*this);
		}
	}

	virtual void ShutdownModule() override
	{
		// unregister player factory
		auto MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->UnregisterPlayerFactory(*this);
		}

#if WITH_EDITOR
		// unregister settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "ImgMedia");
		}
#endif //WITH_EDITOR
	}

private:
	void AddSupportedPlatform(const FGuid& PlatformGuid)
	{
		auto MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");
		if (MediaModule)
		{
			FName PlatformName = MediaModule->GetPlatformName(PlatformGuid);
			if (!PlatformName.IsNone())
			{
				SupportedPlatforms.Add(PlatformName.ToString());
			}
		}
	}

	/** List of supported media file types. */
	TArray<FString> SupportedFileExtensions;

	/** List of platforms that the media player support. */
	TArray<FString> SupportedPlatforms;

	/** List of supported URI schemes. */
	TArray<FString> SupportedUriSchemes;
};


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FImgMediaFactoryModule, ImgMediaFactory);
