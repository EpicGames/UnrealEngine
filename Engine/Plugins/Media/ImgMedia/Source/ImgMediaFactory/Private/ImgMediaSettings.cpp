// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImgMediaSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImgMediaSettings)


/* UImgMediaSettings structors
 *****************************************************************************/

UImgMediaSettings::UImgMediaSettings()
	: DefaultFrameRate(24, 1)
	, BandwidthThrottlingEnabled(true)
	, CacheBehindPercentage(25)
	, CacheSizeGB(1.0f)
	, bEnablePrecachingTimeInterval(false)
	, PrecachingTimeInterval(0.2f)
	, CacheThreads(2)
	, CacheThreadStackSizeKB(128)
	, GlobalCacheSizeGB(1.0f)
	, UseGlobalCache(true)
	, ExrDecoderThreads(0)
	, DefaultProxy(TEXT("proxy"))
	, UseDefaultProxy(false)
{ }

void UImgMediaSettings::PostInitProperties()
{
	Super::PostInitProperties();

	ValidateSettings();
}

#if WITH_EDITOR
void UImgMediaSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	ValidateSettings();

	SettingsChangedDelegate.Broadcast(this);
}

UImgMediaSettings::FOnImgMediaSettingsChanged& UImgMediaSettings::OnSettingsChanged()
{
	return SettingsChangedDelegate;
}

UImgMediaSettings::FOnImgMediaSettingsChanged UImgMediaSettings::SettingsChangedDelegate;
#endif

void UImgMediaSettings::ValidateSettings()
{
	// Global cache size cannot be smaller than the cache size.
	if (GlobalCacheSizeGB < CacheSizeGB)
	{
		GlobalCacheSizeGB = CacheSizeGB;
	}
}

