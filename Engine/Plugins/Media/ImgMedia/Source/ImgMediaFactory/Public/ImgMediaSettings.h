// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Misc/FrameRate.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "ImgMediaSettings.generated.h"

#define UE_API IMGMEDIAFACTORY_API


/**
 * Settings for the ImgMedia module.
 */
UCLASS(MinimalAPI, config=Engine)
class UImgMediaSettings
	: public UObject
{
	GENERATED_BODY()

public:

	/** Default frame rate to use if none was specified in image sequence or media source (default = 1/24). */
	UPROPERTY(config, EditAnywhere, Category=General)
	FFrameRate DefaultFrameRate;

	/** If there is not enough bandwidth to play the media, then lower it (for example by skipping frames) to try and get playback. */
	UPROPERTY(config, EditAnywhere, Category = General)
	bool BandwidthThrottlingEnabled;

public:

	/** Percentage of cache to use for frames behind the play head (default = 25%). */
	UPROPERTY(config, EditAnywhere, Category=Caching, meta=(ClampMin=0.0, ClampMax=100.0))
	float CacheBehindPercentage;

	/**
	 * Maximum memory size of the pre-caching sliding interval (in GB; default = 1 GB).
	 * The pre-caching sliding interval can further be configured by specifying
	 * a time interval (see PrecachingTimeInterval). In that case, the effective
	 * interval will be the minimum of those two settings (size in GB or time span).
	 */
	UPROPERTY(config, EditAnywhere, Category=Caching, meta=(ClampMin=0))
	float CacheSizeGB;

	/**
	 * Enables pre-caching time interval strategy.
	 */
	UPROPERTY(config, EditAnywhere, Category=Caching, meta=(InlineEditConditionToggle))
	bool bEnablePrecachingTimeInterval;

	/**
	 * Specify the pre-caching sliding time interval, in seconds.
	 * The player will pre-cache the specified number of frames corresponding to the time interval relative to the play head.
	 * The cache behind percentage still applies to determine how much frames to pre-cache behind vs ahead of the play head.
	 */
	UPROPERTY(EditAnywhere, Category=Caching, meta=(ClampMin=0, EditCondition="bEnablePrecachingTimeInterval"))
	float PrecachingTimeInterval;
	
	/** Maximum number of image caching threads to use (0 = number of cores, default = 2). */
	UPROPERTY(config, EditAnywhere, Category=Caching, meta=(ClampMin=0))
	int32 CacheThreads;

	/** Size of the stack for each caching thread (in kB; default = 128). */
	UPROPERTY(config, EditAnywhere, Category=Caching, meta=(ClampMin=128), AdvancedDisplay)
	int32 CacheThreadStackSizeKB;

	/** Maximum size of the global look-ahead cache (in GB; default = 1 GB). Must be greater or equal to CacheSizeGB. */
	UPROPERTY(config, EditAnywhere, Category = Caching, meta = (ClampMin = 0))
	float GlobalCacheSizeGB;

	/** Whether to use the global cache or not. */
	UPROPERTY(config, EditAnywhere, Category = Caching)
	bool UseGlobalCache;

public:

	/** Number of worker threads to use when decoding EXR images (0 = auto). */
	UPROPERTY(config, EditAnywhere, Category=EXR, meta=(ClampMin=0))
	uint32 ExrDecoderThreads;

public:
	 
	/** Default constructor. */
	UE_API UImgMediaSettings();

public:

	/**
	 * Get the default media source proxy name tag.
	 *
	 * @return Proxy name tag, or empty string if not set or disabled.
	 */
	FString GetDefaultProxy() const
	{
		if (UseDefaultProxy)
		{
			return DefaultProxy;
		}

		return FString();
	}

	UE_API virtual void PostInitProperties() override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnImgMediaSettingsChanged, const UImgMediaSettings*);

	/** Gets a multicast delegate which is called whenever one of the parameters in this settings object changes. */
	static UE_API FOnImgMediaSettingsChanged& OnSettingsChanged();

protected:
	static UE_API FOnImgMediaSettingsChanged SettingsChangedDelegate;
#endif

private:

	/**
	 * Name of default media source proxy URLs (default = 'proxy').
	 *
	 * Image sequence media sources may contain more than one media source URL. Additional
	 * URLs are called media source proxies, and they are generally used for switching to
	 * lower resolution media content for improved performance during development and testing.
	 *
	 * Each proxy URL has a name associated with it, such as 'proxy', 'lowres', or any
	 * other user defined tag. It is up to the media source to interpret this value and
	 * map it to a media source URL. For example, a media source consisting of a sequence
	 * of uncompressed images may use a proxy name as the name of the sub-directory that
	 * contains proxy content, such as a low-res version of the image sequence.
	 *
	 * When default proxies are enabled via the UseDefaultProxy setting, media players
	 * will first try to locate the proxy content identified by the DefaultProxy tag.
	 * If no such proxy content is available, they will fall back to the media source's
	 * default URL.
	 *
	 * @see UseDefaultProxy
	 */
	UPROPERTY(config, EditAnywhere, Category=Proxies)
	FString DefaultProxy;

	/**
	 * Whether to enable image sequence proxies (default = false).
	 *
	 * Image sequence media sources may contain more than one media source URL. Additional
	 * URLs are called media source proxies, and they are generally used for switching to
	 * lower resolution media content for improved performance during development and testing.
	 *
	 * Each proxy URL has a name associated with it, such as 'proxy', 'lowres', or any
	 * other user defined tag. It is up to the media source to interpret this value and
	 * map it to a media source URL. For example, a media source consisting of a sequence
	 * of uncompressed images may use a proxy name as the name of the sub-directory that
	 * contains proxy content, such as a low-res version of the image sequence.
	 *
	 * When default proxies are enabled via the UseDefaultProxy setting, media players
	 * will first try to locate the proxy content identified by the DefaultProxy tag.
	 * If no such proxy content is available, they will fall back to the media source's
	 * default URL.
	 *
	 * @see DefaultProxy
	 */
	UPROPERTY(config, EditAnywhere, Category=Proxies)
	bool UseDefaultProxy;

	/**
	 * Ensures the settings are valid, and will change them if not.
	 */
	UE_API void ValidateSettings();
};

#undef UE_API
