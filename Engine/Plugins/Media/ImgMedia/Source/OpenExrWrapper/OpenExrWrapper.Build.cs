// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class OpenExrWrapper : ModuleRules
	{
		public OpenExrWrapper(ReadOnlyTargetRules Target) : base(Target)
		{
			bEnableExceptions = true;
			bDisableAutoRTFMInstrumentation = true; // AutoRTFM cannot be used with exceptions
			bUseRTTI = true;

            PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"TimeManagement",
					"TmvMedia",
				});

			bool bLinuxEnabled = Target.Platform == UnrealTargetPlatform.Linux && Target.Architecture == UnrealArch.X64;

            if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) ||
                (Target.Platform == UnrealTargetPlatform.Mac) ||
				bLinuxEnabled)
            {
                AddEngineThirdPartyPrivateStaticDependencies(Target, "Imath");
                AddEngineThirdPartyPrivateStaticDependencies(Target, "UEOpenExr");
                AddEngineThirdPartyPrivateStaticDependencies(Target, "zlib");
            }
            else
            {
                System.Console.WriteLine("OpenExrWrapper does not supported this platform");
            }
        }
	}
}
