// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenExrWrapper.h"

#include <exception>
#include "Containers/UnrealString.h"
#include "Math/IntRect.h"
#include "OpenExrWrapperLog.h"
#include "Logging/StructuredLog.h"

PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
	#include "Imath/ImathBox.h"
	#include "OpenEXR/ImfChannelList.h"
	#include "OpenEXR/ImfCompressionAttribute.h"
	#include "OpenEXR/ImfHeader.h"
	#include "OpenEXR/ImfIntAttribute.h"
	#include "OpenEXR/ImfOutputFile.h"
	#include "OpenEXR/ImfTileDescriptionAttribute.h"
	#include "OpenEXR/ImfRgbaFile.h"
	#include "OpenEXR/ImfStandardAttributes.h"
	#include "OpenEXR/ImfTiledInputFile.h"
	#include "OpenEXR/ImfTiledOutputFile.h"
	#include "OpenEXR/ImfTiledRgbaFile.h"
	#include "OpenEXR/ImfInputFile.h"
	#include "OpenEXR/ImfFrameBuffer.h"
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END

/* FOpenExr
 *****************************************************************************/

void FOpenExr::SetGlobalThreadCount(uint16 ThreadCount)
{
	Imf::setGlobalThreadCount(ThreadCount);
}


/* FRgbaInputFile
 *****************************************************************************/

FRgbaInputFile::FRgbaInputFile(const FString& FilePath)
{
	try
	{
		InputFile = new Imf::RgbaInputFile(TCHAR_TO_ANSI(*FilePath));
	}
	catch (std::exception const& Exception)
	{
		UE_LOGF(LogOpenEXRWrapper, Error, "Cannot load EXR file: %ls", StringCast<TCHAR>(Exception.what()).Get());
		InputFile = nullptr;
	}
}


FRgbaInputFile::FRgbaInputFile(const FString& FilePath, uint16 ThreadCount)
{
	try
	{
		InputFile = new Imf::RgbaInputFile(TCHAR_TO_ANSI(*FilePath), ThreadCount);
	}
	catch (std::exception const& Exception)
	{
		UE_LOGF(LogOpenEXRWrapper, Error, "Cannot load EXR file: %ls", StringCast<TCHAR>(Exception.what()).Get());
		InputFile = nullptr;
	}
}


FRgbaInputFile::~FRgbaInputFile()
{
	delete (Imf::RgbaInputFile*)InputFile;
}


const TCHAR* FRgbaInputFile::GetCompressionName() const
{
	auto CompressionAttribute = ((Imf::RgbaInputFile*)InputFile)->header().findTypedAttribute<Imf::CompressionAttribute>("compression");

	if (CompressionAttribute == nullptr)
	{
		return TEXT("");
	}

	Imf::Compression Compression = CompressionAttribute->value();

	switch (Compression)
	{
	case Imf::NO_COMPRESSION:
		return TEXT("Uncompressed");

	case Imf::RLE_COMPRESSION:
		return TEXT("RLE");

	case Imf::ZIPS_COMPRESSION:
		return TEXT("ZIPS");

	case Imf::ZIP_COMPRESSION:
		return TEXT("ZIP");

	case Imf::PIZ_COMPRESSION:
		return TEXT("PIZ");

	case Imf::PXR24_COMPRESSION:
		return TEXT("PXR24");

	case Imf::B44_COMPRESSION:
		return TEXT("B44");

	case Imf::B44A_COMPRESSION:
		return TEXT("B44A");

	default:
		return TEXT("Unknown");
	}
}


FIntPoint FRgbaInputFile::GetDataWindow() const
{
	Imath::Box2i Win = ((Imf::RgbaInputFile*)InputFile)->dataWindow();

	return FIntPoint(
		Win.max.x - Win.min.x + 1,
		Win.max.y - Win.min.y + 1
	);
}

FIntRect FRgbaInputFile::GetDataWindowRect() const
{
	Imath::Box2i Win;
	if (InputFile != nullptr)
	{
		Win = ((Imf::RgbaInputFile*)InputFile)->dataWindow();
	}

	return FIntRect(Win.min.x, Win.min.y, Win.max.x, Win.max.y);
}

FIntRect FRgbaInputFile::GetDisplayWindow() const
{
	Imath::Box2i Win;
	if (InputFile != nullptr)
	{
		Win = ((Imf::RgbaInputFile*)InputFile)->displayWindow();
	}

	return FIntRect(Win.min.x, Win.min.y, Win.max.x, Win.max.y);
}

FFrameRate FRgbaInputFile::GetFrameRate(const FFrameRate& DefaultValue) const
{
	auto Attribute = ((Imf::RgbaInputFile*)InputFile)->header().findTypedAttribute<Imf::RationalAttribute>("framesPerSecond");

	if (Attribute == nullptr)
	{
		return DefaultValue;
	}

	const Imf::Rational& Value = Attribute->value();

	return FFrameRate(Value.n, Value.d);
}

int32 FRgbaInputFile::GetNumChannels() const
{
	if (InputFile == nullptr)
	{
		return 0;
	}

	Imf::RgbaChannels Channels = ((Imf::RgbaInputFile*)InputFile)->channels();
	int32 NumChannels = 3;
	switch (Channels)
	{
	case Imf::RgbaChannels::WRITE_R:
	case Imf::RgbaChannels::WRITE_G:
	case Imf::RgbaChannels::WRITE_B:
	case Imf::RgbaChannels::WRITE_A:
	case Imf::RgbaChannels::WRITE_Y:
	case Imf::RgbaChannels::WRITE_C:
		NumChannels = 1;
		break;
	case Imf::RgbaChannels::WRITE_YC:
	case Imf::RgbaChannels::WRITE_YA:
		NumChannels = 2;
		break;
	case Imf::RgbaChannels::WRITE_RGB:
	case Imf::RgbaChannels::WRITE_YCA:
		NumChannels = 3;
		break;
	case Imf::RgbaChannels::WRITE_RGBA:
		NumChannels = 4;
		break;
	default:
		break;
	}
	return NumChannels;
}

bool FRgbaInputFile::GetTileSize(FIntPoint& OutTileSize) const
{
	const Imf::TileDescriptionAttribute* TileDescAttr = ((Imf::RgbaInputFile*)InputFile)->header().findTypedAttribute<Imf::TileDescriptionAttribute>("tiles");
	if (TileDescAttr)
	{
		Imf::TileDescription TileDesc = TileDescAttr->value();
		OutTileSize = FIntPoint(TileDesc.xSize, TileDesc.ySize);
	}
	return TileDescAttr != nullptr;
	
}

int32 FRgbaInputFile::GetUncompressedSize() const
{
	const int32 NumChannels = GetNumChannels();
	const int32 ChannelSize = sizeof(int16);
	const FIntPoint Window = GetDataWindow();

	return (Window.X * Window.Y * NumChannels * ChannelSize);
}


bool FRgbaInputFile::IsComplete() const
{
	return ((Imf::RgbaInputFile*)InputFile)->isComplete();
}


bool FRgbaInputFile::HasInputFile() const
{
	return InputFile != nullptr;
}


void FRgbaInputFile::ReadPixels(int32 StartY, int32 EndY)
{
	try
	{
		// Since we convert everything to a coordinate system that goes from 0 to infinity when we GetDataWindow()
		// we need to convert it back to original coordinate system.
		Imath::Box2i Win = ((Imf::RgbaInputFile*)InputFile)->dataWindow();

		((Imf::RgbaInputFile*)InputFile)->readPixels(StartY + Win.min.y, EndY + Win.min.y);
	}
	catch (std::exception const& Exception)
	{
		UE_LOGF(LogOpenEXRWrapper, Error, "Cannot read EXR file: %ls (%ls)",
			ANSI_TO_TCHAR(((Imf::RgbaInputFile*)InputFile)->fileName()),
			StringCast<TCHAR>(Exception.what()).Get());
	}
}


void FRgbaInputFile::SetFrameBuffer(void* Buffer, const FIntPoint& BufferDim)
{
	Imath::Box2i Win = ((Imf::RgbaInputFile*)InputFile)->dataWindow();
	((Imf::RgbaInputFile*)InputFile)->setFrameBuffer((Imf::Rgba*)Buffer - Win.min.x - Win.min.y * BufferDim.X, 1, BufferDim.X);
}

bool FRgbaInputFile::GetIntAttribute(const FString& Name, int32& Value)
{
	bool bIsAttributeFound = false;

	if (InputFile != nullptr)
	{
		const Imf::IntAttribute* Attribute =
			((Imf::RgbaInputFile*)InputFile)->header().
			findTypedAttribute<Imf::IntAttribute>(std::string(TCHAR_TO_ANSI(*Name)));

		if (Attribute != nullptr)
		{
			Value = Attribute->value();
			bIsAttributeFound = true;
		}
	}

	return bIsAttributeFound;
}

/* FExrInputFile
 *****************************************************************************/

FExrInputFile::FExrInputFile(const FString& FilePath, uint16 ThreadCount)
{
	try
	{
		InputFile = new Imf::InputFile(TCHAR_TO_ANSI(*FilePath), ThreadCount);
	}
	catch (std::exception const& Exception)
	{
		UE_LOGFMT(LogOpenEXRWrapper, Error, "Cannot load EXR file: {0}", Exception.what());
		InputFile = nullptr;
	}
}


FExrInputFile::~FExrInputFile()
{
	delete (Imf::InputFile*)InputFile;
}


bool FExrInputFile::HasInputFile() const
{
	return InputFile != nullptr;
}


FIntPoint FExrInputFile::GetDataWindow() const
{
	if (InputFile == nullptr)
	{
		return FIntPoint::ZeroValue;
	}
	Imath::Box2i Win = ((Imf::InputFile*)InputFile)->header().dataWindow();
	return FIntPoint(
		Win.max.x - Win.min.x + 1,
		Win.max.y - Win.min.y + 1
	);
}


void FExrInputFile::ReadPixels(
	void* Buffer,
	const FIntPoint& BufferDim,
	const FString& ChannelR,
	const FString& ChannelG,
	const FString& ChannelB,
	const FString& ChannelA,
	int32 StartY,
	int32 EndY)
{
	if (InputFile == nullptr)
	{
		return;
	}

	try
	{
		Imf::InputFile* File = (Imf::InputFile*)InputFile;
		Imath::Box2i Win = File->header().dataWindow();

		// Validate that our buffer is large enough for the EXR data window.
		const FIntPoint DataDim(Win.max.x - Win.min.x + 1, Win.max.y - Win.min.y + 1);
		if (!ensureMsgf(BufferDim.X >= DataDim.X && BufferDim.Y >= DataDim.Y,
			TEXT("ReadPixels: buffer (%dx%d) is smaller than EXR data window (%dx%d)"),
			BufferDim.X, BufferDim.Y, DataDim.X, DataDim.Y))
		{
			return;
		}

		/**
		 * This function always writes interleaved RGBA half-float output:
		 * [R,G,B,A, R,G,B,A, ...], 8 bytes per pixel.
		 * The strides below define that layout regardless of how channels
		 * are stored in the EXR file (planar, tiled, etc.).
		 */
		const int32 PixelStride = 4 * sizeof(uint16);
		const int32 LineStride = BufferDim.X * PixelStride;

		// Base pointer adjusted for data window origin so that
		// pixel (Win.min.x, Win.min.y) maps to Buffer[0].
		char* Base = (char*)Buffer - Win.min.x * PixelStride - Win.min.y * LineStride;

		Imf::FrameBuffer FrameBuffer;

		// Helper to add a channel slice to the frame buffer.
		// SlotOffset: byte offset of this channel within a pixel (0=R, 2=G, 4=B, 6=A).
		auto AddChannelSlice = [&](const FString& ChannelName, int32 SlotOffset, double SlotFillValue)
		{
			if (!ChannelName.IsEmpty())
			{
				// Only insert if not already present (same channel mapped to multiple slots
				// is handled via post-read copy).
				if (!FrameBuffer.findSlice(StringCast<ANSICHAR>(*ChannelName).Get()))
				{
					FrameBuffer.insert(
						StringCast<ANSICHAR>(*ChannelName).Get(),
						Imf::Slice(
							Imf::HALF,
							Base + SlotOffset,
							PixelStride,
							LineStride,
							1, 1, // x/ySampling
							SlotFillValue
						)
					);
				}
			}
		};

		/**
		 * SlotFillValue is passed as the fillValue argument to Imf::Slice.
		 * OpenEXR uses it to fill the output buffer for any channel that is
		 * registered in the FrameBuffer but absent from the file -- so alpha
		 * is always written as 1.0 for RGB-only EXRs even when "A" is
		 * explicitly requested. The post-read loop below handles only the
		 * separate case where the channel name itself is empty (no file
		 * channel mapped to that output slot at all).
		 */
		AddChannelSlice(ChannelR, 0, 0.0);
		AddChannelSlice(ChannelG, 2, 0.0);
		AddChannelSlice(ChannelB, 4, 0.0);
		AddChannelSlice(ChannelA, 6, 1.0);

		File->setFrameBuffer(FrameBuffer);
		File->readPixels(StartY + Win.min.y, EndY + Win.min.y);

		// Post-read: handle fill slots and grayscale (duplicate channel) copies.
		struct FChannelSlot
		{
			const FString* Name;
			int32 Offset;
		};
		const FChannelSlot Slots[] = {
			{ &ChannelR, 0 },
			{ &ChannelG, 2 },
			{ &ChannelB, 4 },
			{ &ChannelA, 6 },
		};

		// Post-read: fill empty slots and copy duplicate slots (grayscale).
		// Data is interleaved RGBA so we can't use a flat Memset; we work line-by-line
		// and stride over each pixel (4 uint16s = 8 bytes per pixel).
		for (int32 i = 0; i < 4; ++i)
		{
			if (Slots[i].Name->IsEmpty())
			{
				// Fill slot: 0 for RGB, 1.0 (0x3C00) for A.
				const uint16 FillValue = (i == 3) ? 0x3C00 : 0;
				// Build a scratch line with the fill value at every pixel's slot position,
				// then write it for each scanline via a strided loop.
				uint16* PixelPtr = (uint16*)((char*)Buffer + StartY * LineStride + Slots[i].Offset);
				const int32 NumLines = EndY - StartY + 1;
				const int32 NumPixels = BufferDim.X * NumLines;
				for (int32 p = 0; p < NumPixels; ++p)
				{
					*PixelPtr = FillValue;
					PixelPtr += 4; // stride to next pixel's same channel slot
				}
				continue;
			}

			// Check if this channel was already written to a prior slot (grayscale case).
			int32 SourceSlot = -1;
			for (int32 j = 0; j < i; ++j)
			{
				if (!Slots[j].Name->IsEmpty() && *Slots[j].Name == *Slots[i].Name)
				{
					SourceSlot = j;
					break;
				}
			}

			if (SourceSlot >= 0)
			{
				// Copy the already-decoded channel into this duplicate slot, line by line.
				for (int32 y = StartY; y <= EndY; ++y)
				{
					const uint16* SrcPtr = (const uint16*)((const char*)Buffer + y * LineStride + Slots[SourceSlot].Offset);
					uint16* DstPtr = (uint16*)((char*)Buffer + y * LineStride + Slots[i].Offset);
					for (int32 x = 0; x < BufferDim.X; ++x)
					{
						*DstPtr = *SrcPtr;
						SrcPtr += 4;
						DstPtr += 4;
					}
				}
			}
		}
	}
	catch (std::exception const& Exception)
	{
		const ANSICHAR* FileName = InputFile ? ((Imf::InputFile*)InputFile)->fileName() : "";
		UE_LOGFMT(LogOpenEXRWrapper, Error, "Cannot read EXR file: {0} ({1})",
			FileName,
			Exception.what());
	}
}


FBaseOutputFile::FBaseOutputFile(
	const FIntPoint& DisplayWindowMin,
	const FIntPoint& DisplayWindowMax,
	const FIntPoint& DataWindowMin,
	const FIntPoint& DataWindowMax)
{
	OutputFile = nullptr;

	Imath::Box2i EXRDisplayWindow = Imath::Box2i(Imath::V2i(DisplayWindowMin.X, DisplayWindowMin.Y),
		Imath::V2i(DisplayWindowMax.X, DisplayWindowMax.Y));
	Imath::Box2i EXRDataWindow = Imath::Box2i(Imath::V2i(DataWindowMin.X, DataWindowMin.Y),
		Imath::V2i(DataWindowMax.X, DataWindowMax.Y));
	
	Header = new Imf::Header(EXRDisplayWindow, EXRDataWindow, 1, IMATH_NAMESPACE::V2f(0, 0), 1, Imf::INCREASING_Y,
		Imf::NO_COMPRESSION);
}

FBaseOutputFile::~FBaseOutputFile()
{
	if (Header != nullptr)
	{
		delete (Imf::Header*)Header;
	}
	if (OutputFile != nullptr)
	{
		delete (Imf::TiledRgbaOutputFile*)OutputFile;
	}
}

void FBaseOutputFile::AddIntAttribute(const FString& Name, int32 Value)
{
	// Make sure we don't have an output file yet.
	if (OutputFile == nullptr)
	{
		((Imf::Header*)Header)->insert(std::string(StringCast<ANSICHAR>(*Name).Get()), Imf::IntAttribute(Value));
	}
	else
	{
		UE_LOGF(LogOpenEXRWrapper, Error, "Attribute %ls added after calling CreateOutputFile.",
			*Name);
	}
}


FTiledRgbaOutputFile::FTiledRgbaOutputFile(
	const FIntPoint& DisplayWindowMin,
	const FIntPoint& DisplayWindowMax,
	const FIntPoint& DataWindowMin,
	const FIntPoint& DataWindowMax)
	: FBaseOutputFile(DisplayWindowMin, DisplayWindowMax, DataWindowMin, DataWindowMax)
{
}

void FTiledRgbaOutputFile::CreateOutputFile(const FString& FilePath, 
	int32 TileWidth, int32 TileHeight, int32 NumChannels, bool bIsMipsEnabled)
{
	if (OutputFile == nullptr)
	{
		try
		{
			// Get channels.
			Imf::RgbaChannels Channels;
			switch (NumChannels)
			{
			case 1:
				Channels = Imf::WRITE_R;
				break;
			case 2:
				Channels = Imf::WRITE_YC;
				break;
			case 3:
				Channels = Imf::WRITE_RGB;
				break;
			case 4:
				Channels = Imf::WRITE_RGBA;
				break;
			default:
				UE_LOGF(LogOpenEXRWrapper, Error, "Unsupported number of channels %d",
					NumChannels);
				Channels = Imf::WRITE_RGBA;
				break;
			}

			// Create output file.
			OutputFile = new Imf::TiledRgbaOutputFile(StringCast<ANSICHAR>(*FilePath).Get(),
				*((Imf::Header*)Header),
				Channels,
				TileWidth, TileHeight,
				bIsMipsEnabled ? Imf::MIPMAP_LEVELS : Imf::ONE_LEVEL,
				Imf::ROUND_DOWN);
		}
		catch (std::exception const& Exception)
		{
			UE_LOGF(LogOpenEXRWrapper, Error, "Cannot write EXR file: %ls (%ls)",
				*FilePath, StringCast<TCHAR>(Exception.what()).Get());
		}
	}
	else
	{
		UE_LOGF(LogOpenEXRWrapper, Error,
			"Cannot create output file as it has already been created.");
	}
}

int32 FTiledRgbaOutputFile::GetNumberOfMipLevels()
{
	if (OutputFile != nullptr)
	{
		return ((Imf::TiledRgbaOutputFile*)OutputFile)->numLevels();
	}
	else
	{
		UE_LOGF(LogOpenEXRWrapper, Error,
			"GetNumberOfMipLevels failed: CreateOutputFile has not been called yet.");
		return 0;
	}
}

void FTiledRgbaOutputFile::SetFrameBuffer(void* Buffer, const FIntPoint& Stride)
{
	if (OutputFile != nullptr)
	{
		((Imf::TiledRgbaOutputFile*)OutputFile)->setFrameBuffer((Imf::Rgba*)Buffer, Stride.X, Stride.Y);
	}
	else
	{
		UE_LOGF(LogOpenEXRWrapper, Error,
			"SetFrameBuffer failed: CreateOutputFile has not been called yet.");
	}
}

void FTiledRgbaOutputFile::WriteTile(int32 TileX, int32 TileY, int32 MipLevel)
{
	if (OutputFile != nullptr)
	{
		try
		{
			((Imf::TiledRgbaOutputFile*)OutputFile)->writeTile(TileX, TileY, MipLevel);
		}
		catch (std::exception const& Exception)
		{
			UE_LOGF(LogOpenEXRWrapper, Error, "Cannot write EXR file: %ls",
				StringCast<TCHAR>(Exception.what()).Get());
		}
	}
	else
	{
		UE_LOGF(LogOpenEXRWrapper, Error,
			"WriteTile failed: CreateOutputFile has not been called yet.");
	}
}

FTiledOutputFile::FTiledOutputFile(
	const FIntPoint& DisplayWindowMin,
	const FIntPoint& DisplayWindowMax,
	const FIntPoint& DataWindowMin,
	const FIntPoint& DataWindowMax,
	bool bInIsTiled)
	: FBaseOutputFile(DisplayWindowMin, DisplayWindowMax, DataWindowMin, DataWindowMax)
	, bIsTiled(bInIsTiled)
{
	FrameBuffer = new Imf::FrameBuffer;
}

FTiledOutputFile::~FTiledOutputFile()
{
	if (FrameBuffer != nullptr)
	{
		delete (Imf::FrameBuffer*)FrameBuffer;
	}
}

void FTiledOutputFile::AddChannel(const FString& Name)
{
	((Imf::Header*)Header)->channels().insert(StringCast<ANSICHAR>(*Name).Get(), Imf::Channel(Imf::HALF));
}

void FTiledOutputFile::CreateOutputFile(const FString& FilePath,
	int32 TileWidth, int32 TileHeight, bool bIsMipsEnabled, int32 NumThreads)
{
	if (OutputFile == nullptr)
	{
		try
		{
			if (bIsTiled)
			{
				((Imf::Header*)Header)->setTileDescription(Imf::TileDescription(TileWidth, TileHeight,
					bIsMipsEnabled ? Imf::MIPMAP_LEVELS : Imf::ONE_LEVEL));

				// Create output file.
				OutputFile = new Imf::TiledOutputFile(StringCast<ANSICHAR>(*FilePath).Get(),
					*((Imf::Header*)Header), NumThreads);
			}
			else
			{
				// Create output file.
				OutputFile = new Imf::OutputFile(StringCast<ANSICHAR>(*FilePath).Get(),
					*((Imf::Header*)Header), NumThreads);
			}
		}
		catch (std::exception const& Exception)
		{
			UE_LOGF(LogOpenEXRWrapper, Error, "Cannot write EXR file: %ls (%ls)",
				*FilePath, StringCast<TCHAR>(Exception.what()).Get());
		}
	}
	else
	{
		UE_LOGF(LogOpenEXRWrapper, Error,
			"Cannot create output file as it has already been created.");
	}
}

void FTiledOutputFile::AddFrameBufferChannel(const FString& Name, void* Base,
	const FIntPoint& Stride)
{
	((Imf::FrameBuffer*)FrameBuffer)->insert(StringCast<ANSICHAR>(*Name).Get(),
		Imf::Slice(Imf::HALF, (char*)Base, Stride.X, Stride.Y));
}

void FTiledOutputFile::UpdateFrameBufferChannel(const FString& Name, void* Base,
	const FIntPoint& Stride)
{
	Imf::Slice* Slice = ((Imf::FrameBuffer*)FrameBuffer)->findSlice(
		StringCast<ANSICHAR>(*Name).Get());
	if (Slice != nullptr)
	{
		Slice->base = (char*)Base;
		Slice->xStride = Stride.X;
		Slice->yStride = Stride.Y;
	}
	else
	{
		UE_LOGF(LogOpenEXRWrapper, Error, "Could not find frame buffer channel %ls.", *Name);
	}
}

void FTiledOutputFile::SetFrameBuffer()
{
	if (OutputFile != nullptr)
	{
		if (bIsTiled)
		{
			((Imf::TiledOutputFile*)OutputFile)->setFrameBuffer(*((Imf::FrameBuffer*)FrameBuffer));
		}
		else
		{
			((Imf::OutputFile*)OutputFile)->setFrameBuffer(*((Imf::FrameBuffer*)FrameBuffer));
		}
	}
	else
	{
		UE_LOGF(LogOpenEXRWrapper, Error,
			"Cannot set frame buffer as there is no output file.");
	}
}

void FTiledOutputFile::WriteTile(int32 TileX, int32 TileY, int32 MipLevel)
{
	if (bIsTiled)
	{
		if (OutputFile != nullptr)
		{
			try
			{
				((Imf::TiledOutputFile*)OutputFile)->writeTiles(0, TileX, 0, TileY, MipLevel);
			}
			catch (std::exception const& Exception)
			{
				UE_LOGF(LogOpenEXRWrapper, Error, "Cannot write EXR file: %ls",
					StringCast<TCHAR>(Exception.what()).Get());
			}
		}
		else
		{
			UE_LOGF(LogOpenEXRWrapper, Error,
				"WriteTile failed: CreateOutputFile has not been called yet.");
		}
	}
	else
	{
		WriteTiles(0, 0, 0, 0, 0);
	}
}

void FTiledOutputFile::WriteTiles(int32 TileX1, int32 TileX2, int32 TileY1, int32 TileY2, int32 MipLevel)
{
	if (OutputFile != nullptr)
	{
		try
		{
			if (bIsTiled)
			{
				((Imf::TiledOutputFile*)OutputFile)->writeTiles(TileX1, TileX2, TileY1, TileY2, MipLevel);
			}
			else
			{
				Imath::Box2i DataWindow = ((Imf::Header*)Header)->dataWindow();
				((Imf::OutputFile*)OutputFile)->writePixels(DataWindow.max.y - DataWindow.min.y + 1);
			}
		}
		catch (std::exception const& Exception)
		{
			UE_LOGF(LogOpenEXRWrapper, Error, "Cannot write EXR file: %ls",
				StringCast<TCHAR>(Exception.what()).Get());
		}
	}
	else
	{
		UE_LOGF(LogOpenEXRWrapper, Error,
			"WriteTiles failed: CreateOutputFile has not been called yet.");
	}
}

int32 FTiledOutputFile::GetNumberOfMipLevels()
{
	if (OutputFile != nullptr)
	{
		if (bIsTiled)
		{
			return ((Imf::TiledOutputFile*)OutputFile)->numLevels();
		}
		else
		{
			return 1;
		}
	}
	else
	{
		UE_LOGF(LogOpenEXRWrapper, Error,
			"GetNumberOfMipLevels failed: CreateOutputFile has not been called yet.");
		return 0;
	}
}

int32 FTiledOutputFile::GetMipWidth(int32 MipLevel)
{
	if (OutputFile != nullptr)
	{
		if (bIsTiled)
		{
			return ((Imf::TiledOutputFile*)OutputFile)->levelWidth(MipLevel);
		}
		else
		{
			Imath::Box2i DataWindow = ((Imf::Header*)Header)->dataWindow();
			return DataWindow.size().x + 1;
		}
	}
	else
	{
		UE_LOGF(LogOpenEXRWrapper, Error,
			"GetMipWidth failed: CreateOutputFile has not been called yet.");
		return 0;
	}
}

int32 FTiledOutputFile::GetMipHeight(int32 MipLevel)
{
	if (OutputFile != nullptr)
	{
		if (bIsTiled)
		{
			return ((Imf::TiledOutputFile*)OutputFile)->levelHeight(MipLevel);
		}
		else
		{
			Imath::Box2i DataWindow = ((Imf::Header*)Header)->dataWindow();
			return DataWindow.size().y + 1;
		}
	}
	else
	{
		UE_LOGF(LogOpenEXRWrapper, Error,
			"GetMipHeight failed: CreateOutputFile has not been called yet.");
		return 0;
	}
}

int32 FTiledOutputFile::GetNumXTiles(int32 MipLevel)
{
	if (OutputFile != nullptr)
	{
		if (bIsTiled)
		{
			return ((Imf::TiledOutputFile*)OutputFile)->numXTiles(MipLevel);
		}
		else
		{
			return 0;
		}
	}
	else
	{
		UE_LOGF(LogOpenEXRWrapper, Error,
			"GetNumXTiles failed: CreateOutputFile has not been called yet.");
		return 0;
	}
}

int32 FTiledOutputFile::GetNumYTiles(int32 MipLevel)
{
	if (OutputFile != nullptr)
	{
		if (bIsTiled)
		{
			return ((Imf::TiledOutputFile*)OutputFile)->numYTiles(MipLevel);
		}
		else
		{
			return 0;
		}
	}
	else
	{
		UE_LOGF(LogOpenEXRWrapper, Error,
			"GetNumYTiles failed: CreateOutputFile has not been called yet.");
		return 0;
	}
}