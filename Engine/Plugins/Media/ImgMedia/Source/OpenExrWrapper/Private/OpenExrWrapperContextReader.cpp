// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenExrWrapper.h"

#include <exception>
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"

PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
	#include "OpenEXR/openexr_context.h"
	#include "OpenEXR/openexr_part.h"
	#include "OpenEXR/ImfCompressionAttribute.h"
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END

DECLARE_LOG_CATEGORY_EXTERN(LogOpenEXRWrapperHeaderReader, Log, All);
DEFINE_LOG_CATEGORY(LogOpenEXRWrapperHeaderReader);

#define EXR_ATTRIBUTE_COMPRESSION "compression"
#define EXR_ATTRIBUTE_DATAWINDOW "dataWindow"
#define EXR_ATTRIBUTE_CHANNELS "channels"
#define EXR_ATTRIBUTE_TILEDESC "tiles"
#define EXR_ATTRIBUTE_FPS "framesPerSecond"

/* FOpenExrHeaderReader
 *****************************************************************************/

namespace
{
	bool CheckExrResult(const exr_result_t& Result)
	{
		if (Result != EXR_ERR_SUCCESS)
		{
			UE_LOGF(LogOpenEXRWrapperHeaderReader, Error, "OpenExr FOpenExrHeaderReader: Issue extracting data via C Interface.");
		}
		return Result == EXR_ERR_SUCCESS;
	}
}

FOpenExrHeaderReader::FOpenExrHeaderReader(const FString& FilePath) : FileContext(new exr_context_t())
{
	exr_context_initializer_t ContextInitializer = EXR_DEFAULT_CONTEXT_INITIALIZER;
	if (!CheckExrResult(exr_start_read((exr_context_t*)FileContext.Get(), TCHAR_TO_ANSI(*FilePath), &ContextInitializer)))
	{
		FileContext.Reset();
		return;
	}
}

FOpenExrHeaderReader::~FOpenExrHeaderReader()
{
	CheckExrResult(exr_finish((exr_context_t*)FileContext.Get()));
	FileContext.Reset();
}

const TCHAR* FOpenExrHeaderReader::GetCompressionName() const
{
	const exr_attribute_t* Attribute = nullptr;
	CheckExrResult(exr_get_attribute_by_name(*((exr_context_t*)FileContext.Get()), 0, EXR_ATTRIBUTE_COMPRESSION, &Attribute));

	if (Attribute == nullptr)
	{
		return TEXT("");
	}

	switch (Attribute->i)
	{
	case Imf::NO_COMPRESSION:
		return TEXT("Uncompressed");

	case Imf::RLE_COMPRESSION:
		return TEXT("RLE");

	case Imf::ZIPS_COMPRESSION:
		return TEXT("ZIPS");

	case Imf::ZIP_COMPRESSION:
		return TEXT("ZIP");

	case Imf::PIZ_COMPRESSION:
		return TEXT("PIZ");

	case Imf::PXR24_COMPRESSION:
		return TEXT("PXR24");

	case Imf::B44_COMPRESSION:
		return TEXT("B44");

	case Imf::B44A_COMPRESSION:
		return TEXT("B44A");

	default:
		return TEXT("Unknown");
	}
}

FIntPoint FOpenExrHeaderReader::GetDataWindow() const
{
	const exr_attribute_t* Attribute = nullptr;
	CheckExrResult(exr_get_attribute_by_name(*((exr_context_t*)FileContext.Get()), 0, EXR_ATTRIBUTE_DATAWINDOW, &Attribute));
	return FIntPoint(
		Attribute->box2i->max.x - Attribute->box2i->min.x + 1,
		Attribute->box2i->max.y - Attribute->box2i->min.y + 1
	);
}

FFrameRate FOpenExrHeaderReader::GetFrameRate(const FFrameRate& DefaultValue) const
{
	const exr_attribute_t* Attribute = nullptr;
	exr_result_t Result = exr_get_attribute_by_name(*((exr_context_t*)FileContext.Get()), 0, EXR_ATTRIBUTE_FPS, &Attribute);

	if (Result == EXR_ERR_NO_ATTR_BY_NAME || Attribute == nullptr)
	{
		return DefaultValue;
	}

	return FFrameRate(Attribute->rational->num, Attribute->rational->denom);
}

int64 FOpenExrHeaderReader::GetUncompressedSize() const
{
	const int32 TexelSize = GetPixelSize();
	const FIntPoint Window = GetDataWindow();

	return (static_cast<int64>(Window.X) * static_cast<int64>(Window.Y) * static_cast<int64>(TexelSize));
}

int64 FOpenExrHeaderReader::GetUncompressedMipSize(int32 InMipLevel) const
{
	const int32 TexelSize = GetPixelSize();
	const int32 MipLevelDiv = 1 << InMipLevel;
	const FIntPoint MipWindow = (GetDataWindow() / MipLevelDiv).ComponentMax(FIntPoint(1,1));
	return (static_cast<int64>(MipWindow.X) * static_cast<int64>(MipWindow.Y) * static_cast<int64>(TexelSize));
}

int32 FOpenExrHeaderReader::GetNumChannels() const
{
	const exr_attribute_t* Attribute = nullptr;
	CheckExrResult(exr_get_attribute_by_name(*((exr_context_t*)FileContext.Get()), 0, EXR_ATTRIBUTE_CHANNELS, &Attribute));
	return Attribute->chlist->num_channels;
}

int32 FOpenExrHeaderReader::GetPixelSize() const
{
	const exr_attribute_t* Attribute = nullptr;

	CheckExrResult(exr_get_attribute_by_name(*((exr_context_t*)FileContext.Get()), 0, EXR_ATTRIBUTE_CHANNELS, &Attribute));

	int32 PixelSizeInBytes = 0;
	for (int ChannelId = 0; ChannelId < Attribute->chlist->num_channels; ChannelId++)
	{
		const exr_pixel_type_t ChannelType = Attribute->chlist->entries[ChannelId].pixel_type;
		switch (ChannelType)
		{
		// 32 bit, 4 byte
		case EXR_PIXEL_UINT:
		case EXR_PIXEL_FLOAT:
			PixelSizeInBytes += 4;
			break;
		// 16 bit, 2 byte
		case EXR_PIXEL_HALF:
		default:
			PixelSizeInBytes += 2;
			break;
		}
	}
	return PixelSizeInBytes;
}

bool FOpenExrHeaderReader::ContainsMips() const
{
	const exr_attribute_t* Attribute = nullptr;
	exr_result_t Result = exr_get_attribute_by_name(*((exr_context_t*)FileContext.Get()), 0, EXR_ATTRIBUTE_TILEDESC, &Attribute);
	if (Result == EXR_ERR_SUCCESS && Attribute)
	{
		exr_tile_level_mode_t TileMipMode = EXR_GET_TILE_LEVEL_MODE(*Attribute->tiledesc);
		return TileMipMode == EXR_TILE_MIPMAP_LEVELS;
	}
	return false;
}

int32 FOpenExrHeaderReader::GetNumMipLevels() const
{
	int32_t NumLevelsX = 0, NumLevelsY = 0;
	exr_result_t Result = exr_get_tile_levels(*((exr_context_t*)FileContext.Get()), 0, &NumLevelsX, &NumLevelsY);
	if (Result == EXR_ERR_SUCCESS)
	{
		return FMath::Min(NumLevelsX, NumLevelsY);
	}
	return 1;
}

int32 FOpenExrHeaderReader::CalculateNumMipLevels(const FIntPoint& NumTiles) const
{
	const exr_attribute_t* Attribute = nullptr;
	exr_result_t Result = exr_get_attribute_by_name(*((exr_context_t*)FileContext.Get()), 0, EXR_ATTRIBUTE_TILEDESC, &Attribute);
	if (Result == EXR_ERR_SUCCESS && Attribute)
	{
		exr_tile_round_mode_t MipRoundMode = EXR_GET_TILE_ROUND_MODE(*Attribute->tiledesc);
		int32 MinTileRes = FMath::Min(NumTiles.X, NumTiles.Y);
		int32 NumMipLevels = (MipRoundMode == exr_tile_round_mode_t::EXR_TILE_ROUND_DOWN) ? FMath::FloorLog2(MinTileRes) : FMath::CeilLogTwo(MinTileRes);
		return NumMipLevels + 1;
	}
	return 1;
}

bool FOpenExrHeaderReader::IsOptimizedForGpu() const
{
	const exr_attribute_t* Attribute = nullptr;

	CheckExrResult(exr_get_attribute_by_name(*((exr_context_t*)FileContext.Get()), 0, EXR_ATTRIBUTE_CHANNELS, &Attribute));

	for (int ChannelId = 0; ChannelId < Attribute->chlist->num_channels; ChannelId++)
	{
		const exr_pixel_type_t ChannelType = Attribute->chlist->entries[ChannelId].pixel_type;
		if (ChannelType != EXR_PIXEL_HALF)
		{
			return false;
		}
	}
	return Attribute->chlist->num_channels <= 4;
}

bool FOpenExrHeaderReader::GetTileSize(FIntPoint& OutTileSize) const
{
	const exr_attribute_t* Attribute = nullptr;
	exr_result_t Result = exr_get_attribute_by_name(*((exr_context_t*)FileContext.Get()), 0, EXR_ATTRIBUTE_TILEDESC, &Attribute);
	if (Result == EXR_ERR_SUCCESS && Attribute)
	{
		OutTileSize = FIntPoint(Attribute->tiledesc->x_size, Attribute->tiledesc->y_size);
	}
	return Attribute != nullptr;
}

bool FOpenExrHeaderReader::HasInputFile() const
{
	return FileContext != nullptr;
}

void FOpenExrHeaderReader::GetChannelNames(TArray<FString>& OutChannelNames) const
{
	OutChannelNames.Reset();

	if (!FileContext.IsValid())
	{
		return;
	}

	const exr_attribute_t* Attribute = nullptr;
	exr_result_t Result = exr_get_attribute_by_name(*((exr_context_t*)FileContext.Get()), 0, "channels", &Attribute);

	if (Result != EXR_ERR_SUCCESS || Attribute == nullptr)
	{
		return;
	}

	for (int32 ChannelId = 0; ChannelId < Attribute->chlist->num_channels; ChannelId++)
	{
		OutChannelNames.Add(ANSI_TO_TCHAR(Attribute->chlist->entries[ChannelId].name.str));
	}
}

bool FOpenExrHeaderReader::GetIntAttribute(const FString& Name, int32& Value)
{
	const exr_attribute_t* Attribute = nullptr;
	exr_result_t Result = exr_get_attribute_by_name(*((exr_context_t*)FileContext.Get()), 0, TCHAR_TO_ANSI(*Name), &Attribute);
	if (Result == EXR_ERR_SUCCESS && Attribute)
	{
		Value = Attribute->i;
		return true;
	}
	return false;
}



