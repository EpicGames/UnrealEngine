// Copyright Epic Games, Inc. All Rights Reserved.

#include "ITmvMediaModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "OpenExrWrapperLog.h"
#include "OpenExrWrapperTmvEncoderFactory.h"

DEFINE_LOG_CATEGORY(LogOpenEXRWrapper);

/** Implementation of the OpenExrWrapper module. */
class FOpenExrWrapperModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override
	{
		EncoderFactory = MakeShared<FOpenExrWrapperTmvEncoderFactory>();
		ITmvMediaModule::GetOrLoad().RegisterEncoderFactory(EncoderFactory);
	}

	virtual void ShutdownModule() override
	{
		if (ITmvMediaModule* TmvMediaModule = ITmvMediaModule::Get())
		{
			TmvMediaModule->UnregisterEncoderFactory(EncoderFactory);
		}
		EncoderFactory.Reset();
	}
	//~ End IModuleInterface

private:
	/** OpenExr TmvEncoder factory instance */
	TSharedPtr<FOpenExrWrapperTmvEncoderFactory, ESPMode::ThreadSafe> EncoderFactory;
};

IMPLEMENT_MODULE(FOpenExrWrapperModule, OpenExrWrapper);