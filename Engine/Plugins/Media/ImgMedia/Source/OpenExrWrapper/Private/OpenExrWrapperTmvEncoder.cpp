// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenExrWrapperTmvEncoder.h"

#include "OpenExrWrapper.h"
#include "OpenExrWrapperLog.h"
#include "SampleConverter/TmvMediaFrameMipBuffer.h"
#include "Utils/TmvMediaMessageContext.h"
#include "Utils/TmvMediaUtils.h"

#define LOCTEXT_NAMESPACE "OpenExrTmvEncoder"

// Encoder utility functions
namespace UE::OpenExrWrapper::Encoder
{
	/** Utility function to apply tint to a row of pixels in place. */
	void TintRowInplace(FFloat16* InRowBuffer, uint32 InWidth, float TintFactor)
	{
		for (uint32 Index = 0; Index < InWidth; ++Index)
		{
			InRowBuffer[Index] = (InRowBuffer[Index].GetFloat() + TintFactor) * 0.5f;
		}
	}

	/** Utility function to apply tint on the given mip buffer in place. */
	void TintInplace(const FTmvMediaFrameMipBufferHandle& InMipBuffer, const TArrayView<FLinearColor>& InMipLevelTints)
	{
		const auto& MipInfo = InMipBuffer->GetMipInfoRef();
		if (MipInfo.NumComponents != MipInfo.Planes.Num())
		{
			UE_LOGF(LogOpenEXRWrapper, Error, "Non planar tinting is not supported.");
			return; // non-planar is not supported (but could be if needed, just need to implement it).
		}

		// Get tint colour.
		const FLinearColor TintColor = (InMipLevelTints.Num() > 0) ? InMipLevelTints[MipInfo.MipLevel % InMipLevelTints.Num()] : FLinearColor::White;

		for (int32 ComponentIndex = 0; ComponentIndex < MipInfo.NumComponents; ++ComponentIndex)
		{
			// Don't tint alpha.
			if (ComponentIndex == 3)
			{
				continue;
			}
			const float TintFactor = TintColor.Component(ComponentIndex);

			const int32 PlaneIndex = MipInfo.GetPlaneIndexForComponent(ComponentIndex);

			if (!MipInfo.Planes.IsValidIndex(PlaneIndex))
			{
				UE_LOGF(LogOpenEXRWrapper, Error, "Tint: Invalid plane index %d for component %d.", PlaneIndex, ComponentIndex);
				continue;
			}

			const FTmvMediaFramePlaneInfo& PlaneInfo = MipInfo.Planes[PlaneIndex];

			if (PlaneInfo.BitDepth != 16 || PlaneInfo.Type != ETmvMediaFrameComponentType::Float)
			{
				UE_LOGF(LogOpenEXRWrapper, Error, "Tint: Plane %d for component %d is not Float16.", PlaneIndex, ComponentIndex);
				continue;
			}

			uint8* PlaneBuffer = static_cast<uint8*>(InMipBuffer->GetPlaneBufferForComponent(ComponentIndex));

			if (PlaneBuffer == nullptr)
			{
				UE_LOGF(LogOpenEXRWrapper, Error, "Tint: Null plane buffer for component %d.", ComponentIndex);
				continue; 
			}

			for (uint32 PlaneY = 0; PlaneY < PlaneInfo.Height; ++PlaneY)
			{
				FFloat16* RowBuffer = reinterpret_cast<FFloat16*>(PlaneBuffer + PlaneY * PlaneInfo.Stride);
				TintRowInplace(RowBuffer, PlaneInfo.Width, TintFactor);
			}
		}
	}
}

FOpenExrWrapperTmvEncoder::FOpenExrWrapperTmvEncoder(const FOpenExrWrapperTmvEncoderOptions& InEncoderOptions)
	: EncoderOptions(InEncoderOptions)
{
}

FOpenExrWrapperTmvEncoder::~FOpenExrWrapperTmvEncoder() = default;

ETmvMediaEncoderResult FOpenExrWrapperTmvEncoder::RequestMipInfos(
	const FTmvMediaFrameTimeInfo& InTimeInfo,
	const FTmvMediaEncoderMipInfo& InFrameInfo,
	TArray<FTmvMediaFrameMipInfo>& OutFrameMipInfo,
	FTmvMediaMessageContext* OutMessageContext)
{
	if (InFrameInfo.Width <= 0 || InFrameInfo.Height <= 0)
	{
		FText Message = FText::Format(
		LOCTEXT("InvalidFrameDim", "Invalid frame dimensions ({0} x {1})"),
			FText::AsNumber(InFrameInfo.Width), FText::AsNumber(InFrameInfo.Height));
		UE_TMV_MEDIA_MESSAGE_LOG(OutMessageContext, LogOpenEXRWrapper, Error, TEXT("RequestMipInfos"), Message);
		return ETmvMediaEncoderResult::Fail;
	} 

	const int32 MaxNumMips = FTmvMediaUtils::GetMaxMipCountFromDimensions(InFrameInfo.Width, InFrameInfo.Height);
	const int32 NumMips = InFrameInfo.bEnableMips ? MaxNumMips : 1;
	OutFrameMipInfo.Reserve(NumMips);

	for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
	{
		// See levelSize and LevelRoundingMode.
		// By default, it calculates the mip size by truncating (ROUND_DOWN), which is what we do here.
		const int32 MipWidth = FMath::Max(InFrameInfo.Width >> MipIndex, 1);
		const int32 MipHeight = FMath::Max(InFrameInfo.Height >> MipIndex, 1);
		FTmvMediaFrameMipInfo MipInfo;
		MipInfo.MipLevel = MipIndex;
		MipInfo.Width = MipWidth;
		MipInfo.Height = MipHeight;
		MipInfo.ColorModel = ETmvMediaFrameColorModel::RGB;

		// Setup the default color management for this encoder (RGB, linear)
		MipInfo.ColorInfo.Encoding = UE::Color::EEncoding::Linear;
		MipInfo.ColorInfo.ColorSpace = UE::Color::EColorSpace::sRGB;
		MipInfo.ColorInfo.YuvMatrix = ETmvMediaFrameColorMatrix::Identity;
		MipInfo.ColorInfo.YuvMatrixRange = ETmvMediaFrameColorMatrixRange::Full;

		// Apply overrides from encoder settings.
		MipInfo.ColorInfo.ApplyOverrides(InFrameInfo.ColorInfo);

		MipInfo.NumComponents = EncoderOptions.bRemoveAlphaChannel ? 3 : 4;

		if (EncoderOptions.IsTilingEnabled())
		{
			MipInfo.TileWidth = EncoderOptions.TileSizeX;
			MipInfo.TileHeight = EncoderOptions.TileSizeY;
			// See precalculateTileInfo -> calculateNumTiles
			const int32 NumTilesX = (MipWidth + EncoderOptions.TileSizeX - 1) / EncoderOptions.TileSizeX;
			const int32 NumTilesY = (MipHeight + EncoderOptions.TileSizeY - 1) / EncoderOptions.TileSizeY;
			MipInfo.NumTiles = FIntPoint(NumTilesX, NumTilesY);
		}
		else
		{
			MipInfo.TileWidth = MipWidth;
			MipInfo.TileHeight = MipHeight;
			MipInfo.NumTiles = FIntPoint(1, 1);
		}

		// Ask the converter to give us a buffer in scan line format.
		// Converter doesn't support tiling currently (too many variations). The encoder will be expected to do the tiling. 
		MipInfo.Layout = ETmvMediaFrameBufferLayout::ScanLine; //ETmvMediaFrameBufferLayout::Tiled;	// converter doesn't support tiling for now.
		MipInfo.Planes.Reserve(MipInfo.NumComponents);
		for (int32 ComponentIndex = 0; ComponentIndex < MipInfo.NumComponents; ++ComponentIndex)
		{
			constexpr int32 NumBytesPerComp = 2;	// Float16

			FTmvMediaFramePlaneInfo PlaneInfo;
			PlaneInfo.NumComponents = 1;
			PlaneInfo.BitDepth = 16; // Float16
			PlaneInfo.Width = MipWidth;
			PlaneInfo.Height = MipHeight;
			PlaneInfo.Stride = MipWidth * NumBytesPerComp;
			PlaneInfo.NumLines = MipHeight;
			PlaneInfo.WidthRatio = 1;
			PlaneInfo.HeightRatio = 1;
			PlaneInfo.Type = ETmvMediaFrameComponentType::Float;;
			MipInfo.Planes.Add(PlaneInfo);
		}
		OutFrameMipInfo.Add(MipInfo);
	}
	return ETmvMediaEncoderResult::Success;
}

ETmvMediaEncoderResult FOpenExrWrapperTmvEncoder::Encode(
	const FTmvMediaFrameTimeInfo& InTimeInfo,
	ITmvMediaEncoderAccessUnit& InAccessUnit,
	TArrayView<FTmvMediaEncoderMipRequest> InMipRequests,
	FTmvMediaMessageContext* OutMessageContext)
{
	// TODO: GetImageParameters. See what DataWindow is and how it is used.

	if (InMipRequests.IsEmpty() || !InMipRequests[0].MipBuffer.IsValid())
	{
		return ETmvMediaEncoderResult::Fail;
	}

	// Names for our channels.
	const TArray<FString, TInlineAllocator<4>> ChannelNames = { TEXT("R"), TEXT("G"), TEXT("B"), TEXT("A")};

	const FTmvMediaFrameMipInfo& Mip0Info = InMipRequests[0].MipBuffer->GetMipInfoRef();
	ensure(Mip0Info.MipLevel == 0);

	if (Mip0Info.Width <= 0 || Mip0Info.Height <= 0)
	{
		FText Message = FText::Format(
			LOCTEXT("InvalidMip0Dim", "Invalid mip0 dimensions ({0} x {1})"),
			FText::AsNumber(Mip0Info.Width), FText::AsNumber(Mip0Info.Height));
		UE_TMV_MEDIA_MESSAGE_LOG(OutMessageContext, LogOpenEXRWrapper, Error, TEXT("Encode"), Message);
		return ETmvMediaEncoderResult::Fail;
	}

	// For now, we only support RGB and RGBA formats.
	if (Mip0Info.NumComponents > ChannelNames.Num())
	{
		FText Message = FText::Format(
			LOCTEXT("InvalidNumComponents", "The input image has too many components ({0}), maximum supported number of components: {1}"),
			FText::AsNumber(Mip0Info.NumComponents), FText::AsNumber(ChannelNames.Num()));
		UE_TMV_MEDIA_MESSAGE_LOG(OutMessageContext, LogOpenEXRWrapper, Error, TEXT("Encode"), Message);
		return ETmvMediaEncoderResult::Fail;
	}

	const FIntRect DataWindow = FIntRect(0, 0, Mip0Info.Width - 1, Mip0Info.Height - 1);
	const FIntRect DisplayWindow = FIntRect(0, 0, Mip0Info.Width - 1, Mip0Info.Height - 1);

	{
		// Create tiled exr file.
		FTiledOutputFile OutFile(DisplayWindow.Min, DisplayWindow.Max,DataWindow.Min, DataWindow.Max, EncoderOptions.IsTilingEnabled());

		// Why add the channels in reverse order?
		for (int32 ComponentIndex = Mip0Info.NumComponents - 1; ComponentIndex >= 0; --ComponentIndex)
		{
			OutFile.AddChannel(ChannelNames[ComponentIndex]);
		}

		const bool bEnableMips = InMipRequests.Num() > 1;
		FString Filename = InAccessUnit.GetFilename();
		OutFile.CreateOutputFile(Filename, Mip0Info.TileWidth, Mip0Info.TileHeight, bEnableMips, 1);

		// Why add null buffers here?
		for (int32 ComponentIndex = Mip0Info.NumComponents - 1; ComponentIndex >= 0; --ComponentIndex)
		{
			int32 PlaneIndex = Mip0Info.GetPlaneIndexForComponent(ComponentIndex);
			if (!ensure(Mip0Info.Planes.IsValidIndex(PlaneIndex)))
			{
				continue;
			}

			const FTmvMediaFramePlaneInfo& PlaneInfo = Mip0Info.Planes[PlaneIndex];
			const FIntPoint PlaneStride(PlaneInfo.GetBytesPerComponent(), PlaneInfo.Stride);
			OutFile.AddFrameBufferChannel(ChannelNames[ComponentIndex], nullptr, PlaneStride);
		}

		for (int32 MipLevel = 0; MipLevel < InMipRequests.Num(); ++MipLevel)
		{
			FTmvMediaFrameMipBufferHandle MipBuffer = InMipRequests[MipLevel].MipBuffer;
			if (!MipBuffer)
			{
				continue;
			}
			ensure(MipBuffer->GetMipInfoRef().MipLevel == MipLevel);	// mip level mismatch.
			const FTmvMediaFrameMipInfo& MipInfo = MipBuffer->GetMipInfoRef();

			if (EncoderOptions.bEnableMipLevelTint)
			{
				UE::OpenExrWrapper::Encoder::TintInplace(MipBuffer, EncoderOptions.MipLevelTints);
			}

			const int32 NumComponents = FMath::Min(MipInfo.NumComponents, ChannelNames.Num());

			for (int32 ComponentIndex = 0;  ComponentIndex < NumComponents; ++ComponentIndex)
			{
				void* PlaneBuffer = MipBuffer->GetPlaneBufferForComponent(ComponentIndex);
				if (PlaneBuffer == nullptr)
				{
					FText Message = FText::Format(
						LOCTEXT("NullPlaneBuffer", "Null plane buffer for component {0} (mip {1})"),
						FText::AsNumber(ComponentIndex), FText::AsNumber(MipLevel));
					UE_TMV_MEDIA_MESSAGE_LOG(OutMessageContext, LogOpenEXRWrapper, Error, TEXT("Encode"), Message);
					return ETmvMediaEncoderResult::Fail;
				}

				const int32 PlaneIndex = MipInfo.GetPlaneIndexForComponent(ComponentIndex);

				if (!ensure(MipInfo.Planes.IsValidIndex(PlaneIndex)))
				{
					continue;
				}

				const FTmvMediaFramePlaneInfo& PlaneInfo = MipInfo.Planes[PlaneIndex];
				const FIntPoint PlaneStride(PlaneInfo.GetBytesPerComponent(), PlaneInfo.Stride);

				OutFile.UpdateFrameBufferChannel(ChannelNames[ComponentIndex], PlaneBuffer, PlaneStride);
			}

			OutFile.SetFrameBuffer();
			int32 MaxTileX = FMath::Max(OutFile.GetNumXTiles(MipLevel) - 1, 0);
			int32 MaxTileY = FMath::Max(OutFile.GetNumYTiles(MipLevel) - 1, 0);
			OutFile.WriteTiles(0, MaxTileX, 0, MaxTileY, MipLevel);
		}
	}

	return ETmvMediaEncoderResult::Success;
}

#undef LOCTEXT_NAMESPACE
