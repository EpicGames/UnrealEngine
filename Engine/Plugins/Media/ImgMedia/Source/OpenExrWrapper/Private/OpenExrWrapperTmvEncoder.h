// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OpenExrWrapperTmvEncoderOptions.h"
#include "Encoder/ITmvMediaEncoder.h"

/**
 * Implementation of a TMV encoder for the OpenExr.
 */
class FOpenExrWrapperTmvEncoder : public ITmvMediaEncoder
{
public:
	FOpenExrWrapperTmvEncoder(const FOpenExrWrapperTmvEncoderOptions& InEncoderOptions);
	virtual ~FOpenExrWrapperTmvEncoder() override;

	//~ Begin ITmvMediaEncoder
	virtual ETmvMediaEncoderResult RequestMipInfos(
		const FTmvMediaFrameTimeInfo& InTimeInfo,
		const FTmvMediaEncoderMipInfo& InFrameInfo,
		TArray<FTmvMediaFrameMipInfo>& OutFrameMipInfo,
		FTmvMediaMessageContext* OutMessageContext) override;

	virtual ETmvMediaEncoderResult Encode(
		const FTmvMediaFrameTimeInfo& InTimeInfo,
		ITmvMediaEncoderAccessUnit& InAccessUnit,
		TArrayView<FTmvMediaEncoderMipRequest> InMipRequests,
		FTmvMediaMessageContext* OutMessageContext) override;
	//~ End ITmvMediaEncoder

private:
	/** Current encoder options */
	FOpenExrWrapperTmvEncoderOptions EncoderOptions;
};