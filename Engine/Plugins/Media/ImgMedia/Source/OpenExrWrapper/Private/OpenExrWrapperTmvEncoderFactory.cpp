// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenExrWrapperTmvEncoderFactory.h"

#include "OpenExrWrapperLog.h"
#include "OpenExrWrapperTmvEncoder.h"
#include "OpenExrWrapperTmvEncoderOptions.h"
#include "StructUtils/InstancedStruct.h"

#define LOCTEXT_NAMESPACE "OpenExrWrapperTmvEncoderFactory"

const FLazyName FOpenExrWrapperTmvEncoderFactory::EncoderName(TEXT("OpenExr"));

FName FOpenExrWrapperTmvEncoderFactory::GetName() const
{
	return EncoderName;
}

FText FOpenExrWrapperTmvEncoderFactory::GetDisplayName() const
{
	return LOCTEXT("FactoryDisplayName", "OpenExr");
}

void FOpenExrWrapperTmvEncoderFactory::GetEncoderOptions(TInstancedStruct<FTmvMediaEncoderOptions>& OutOptions) const
{
	OutOptions.InitializeAs<FOpenExrWrapperTmvEncoderOptions>();
}

TSharedPtr<ITmvMediaEncoder, ESPMode::ThreadSafe> FOpenExrWrapperTmvEncoderFactory::CreateEncoder(const FString& InCodecFormat, const TInstancedStruct<FTmvMediaEncoderOptions>& InOptions)
{
	if (const FOpenExrWrapperTmvEncoderOptions* EncoderOptions = InOptions.GetPtr<FOpenExrWrapperTmvEncoderOptions>())
	{
		return MakeShared<FOpenExrWrapperTmvEncoder, ESPMode::ThreadSafe>(*EncoderOptions);
	}

	UE_LOGF(LogOpenEXRWrapper, Error, "Unexpected encoder options (%ls).",
		InOptions.GetScriptStruct() ? *InOptions.GetScriptStruct()->GetName() : TEXT("<null>"));
	return nullptr;
}

#undef LOCTEXT_NAMESPACE