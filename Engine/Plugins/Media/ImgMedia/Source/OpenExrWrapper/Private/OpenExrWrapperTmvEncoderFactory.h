// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Encoder/ITmvMediaEncoderFactory.h"

/** Implementation of OpenExr Tmv Encoder Factory. */ 
class FOpenExrWrapperTmvEncoderFactory : public ITmvMediaEncoderFactory
{
public:
	virtual ~FOpenExrWrapperTmvEncoderFactory() override = default;

	//~ Begin ITmvMediaEncoderFactory
	virtual FName GetName() const override;
	virtual FText GetDisplayName() const override;
	virtual bool SupportsMemoryAccessUnit() const override { return false; }
	virtual void GetEncoderOptions(TInstancedStruct<FTmvMediaEncoderOptions>& OutOptions) const override;
	virtual TSharedPtr<ITmvMediaEncoder, ESPMode::ThreadSafe> CreateEncoder(const FString& InCodecFormat, const TInstancedStruct<FTmvMediaEncoderOptions>& InOptions) override;
	//~ End ITmvMediaEncoderFactory

	/** Encoder Name. */
	static const FLazyName EncoderName;
};