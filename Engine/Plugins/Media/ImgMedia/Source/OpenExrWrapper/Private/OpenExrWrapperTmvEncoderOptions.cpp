// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenExrWrapperTmvEncoderOptions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OpenExrWrapperTmvEncoderOptions)

FOpenExrWrapperTmvEncoderOptions::FOpenExrWrapperTmvEncoderOptions()
{
	MipLevelTints.Add(FLinearColor::Red);
	MipLevelTints.Add(FLinearColor::Green);
	MipLevelTints.Add(FLinearColor::Blue);
	MipLevelTints.Add(FLinearColor::Yellow);
	MipLevelTints.Add(FLinearColor(0.0f, 1.0f, 1.0f));
	MipLevelTints.Add(FLinearColor(1.0f, 0.0f, 1.0f));
}


