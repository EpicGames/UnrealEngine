// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Encoder/TmvMediaEncoderOptions.h"
#include "Math/Color.h"
#include "OpenExrWrapperTmvEncoderFactory.h"

#include "OpenExrWrapperTmvEncoderOptions.generated.h"

/**
 * OpenExr Tmv Encoder options.
 */
USTRUCT(DisplayName = "OpenExr Tmv Encoder Options")
struct FOpenExrWrapperTmvEncoderOptions : public FTmvMediaEncoderOptions
{
	GENERATED_BODY()

	FOpenExrWrapperTmvEncoderOptions();
	virtual ~FOpenExrWrapperTmvEncoderOptions() override = default;

	//~ Begin FTmvMediaEncoderOptions
	virtual FName GetEncoderName() const override { return FOpenExrWrapperTmvEncoderFactory::EncoderName;}
	virtual FString GetFileSequenceExtension() const override { return TEXT("exr"); }
	//~ End FTmvMediaEncoderOptions

	/** Returns true if tiling is enabled. */
	bool IsTilingEnabled() const
	{
		return bEnableTiling && TileSizeX > 0 && TileSizeY > 0;
	}

	/** If checked, then enable tiling. */
	UPROPERTY(EditAnywhere, Category = Tiles)
	bool bEnableTiling = true;
	
	/** Width of a tile in pixels. If 0, then do not make tiles. */
	UPROPERTY(EditAnywhere, Category = Tiles, meta = (EditCondition = "bEnableTiling", ClampMin = "0.0"))
	int32 TileSizeX = 256;

	/** Height of a tile in pixels. If 0, then do not make tiles. */
	UPROPERTY(EditAnywhere, Category = Tiles, meta = (EditCondition = "bEnableTiling", ClampMin = "0.0"))
	int32 TileSizeY = 256;

	/** Number of worker thread to encode tiles (0 = automatically use the available number of cores). */
	UPROPERTY(EditAnywhere, Category = "Advanced", meta = (ClampMin = "0.0"))
	int32 NumThreads = 0;

	/** 
	* This option removes alpha channel.
	* Since alpha channel adds 25% reading cost, it is suggested to have alpha channel removed if it is not used. 
	*/
	UPROPERTY(EditAnywhere, Category = Channels)
	bool bRemoveAlphaChannel = false;

	/** Tint each mip level a different colour to help with debugging. */
	UPROPERTY(EditAnywhere, Category = Debug)
	bool bEnableMipLevelTint = false;

	/** Color to tint each mip level. */
	UPROPERTY(EditAnywhere, Category = Debug)
	TArray<FLinearColor> MipLevelTints;
};
