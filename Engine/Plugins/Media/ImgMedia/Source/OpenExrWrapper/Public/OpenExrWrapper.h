// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/FrameRate.h"
#include "Math/IntPoint.h"

#define UE_API OPENEXRWRAPPER_API

class FString;


class FOpenExr
{
public:

	static UE_API void SetGlobalThreadCount(uint16 ThreadCount);
};


class FRgbaInputFile
{
public:

	UE_API FRgbaInputFile(const FString& FilePath);
	UE_API FRgbaInputFile(const FString& FilePath, uint16 ThreadCount);
	UE_API ~FRgbaInputFile();

public:

	UE_API const TCHAR* GetCompressionName() const;
	UE_API FIntPoint GetDataWindow() const;
	UE_API FIntRect GetDataWindowRect() const;
	UE_API FIntRect GetDisplayWindow() const;
	UE_API FFrameRate GetFrameRate(const FFrameRate& DefaultValue) const;
	UE_API int32 GetUncompressedSize() const;
	UE_API int32 GetNumChannels() const;

	// Gets tile dimensions. Returns false if image has no tiles.
	UE_API bool GetTileSize(FIntPoint& OutTileSize) const;

	UE_API bool IsComplete() const;
	UE_API bool HasInputFile() const;
	UE_API void ReadPixels(int32 StartY, int32 EndY);
	UE_API void SetFrameBuffer(void* Buffer, const FIntPoint& Stride);

	/**
	 * Get an attribute from the image.
	 *
	 * @param Name		Name of attribute.
	 * @param Value		Will be set to the value of the attribute if the attribute is found.
	 *					Will NOT be set if the attribute is not found.
	 * @return			True if the attribute was found, false otherwise.
	 */
	UE_API bool GetIntAttribute(const FString& Name, int32& Value);

private:

	void* InputFile;
};

/**
 * Reads EXR files with per-channel control using the general Imf::InputFile API.
 * Unlike FRgbaInputFile, this allows selecting specific named channels for each RGBA output slot.
 */
class FExrInputFile
{
public:

	UE_API FExrInputFile(const FString& FilePath, uint16 ThreadCount);
	UE_API ~FExrInputFile();

public:

	UE_API bool HasInputFile() const;
	UE_API FIntPoint GetDataWindow() const;

	/**
	 * Read specific channels into an RGBA half-float (uint16) interleaved buffer.
	 *
	 * Each channel parameter is a full EXR channel name (e.g., "R", "beauty.G", "Zdepth.Z").
	 * Empty string means fill with a default value (0 for RGB slots, 1 for A slot).
	 * The same channel name may be used for multiple slots (e.g., all RGB = "Zdepth.Z" for grayscale).
	 *
	 * @param Buffer		Destination buffer, must hold BufferDim.X * BufferDim.Y * 4 * sizeof(uint16) bytes.
	 * @param BufferDim		Width and height of the buffer (used for stride calculation).
	 * @param ChannelR		EXR channel name to read into the R slot.
	 * @param ChannelG		EXR channel name to read into the G slot.
	 * @param ChannelB		EXR channel name to read into the B slot.
	 * @param ChannelA		EXR channel name to read into the A slot.
	 * @param StartY		First scanline to read (0-based, relative to data window).
	 * @param EndY			Last scanline to read (inclusive, 0-based, relative to data window).
	 */
	UE_API void ReadPixels(
		void* Buffer,
		const FIntPoint& BufferDim,
		const FString& ChannelR,
		const FString& ChannelG,
		const FString& ChannelB,
		const FString& ChannelA,
		int32 StartY,
		int32 EndY);

private:

	void* InputFile;
};


class FOpenExrHeaderReader
{
public:

	UE_API FOpenExrHeaderReader(const FString& FilePath);
	UE_API ~FOpenExrHeaderReader();

public:

	UE_API const TCHAR* GetCompressionName() const;
	UE_API FIntPoint GetDataWindow() const;
	UE_API FFrameRate GetFrameRate(const FFrameRate& DefaultValue) const;
	UE_API int64 GetUncompressedSize() const;
	UE_API int64 GetUncompressedMipSize(int32 InMipLevel) const;
	UE_API int32 GetNumChannels() const;
	UE_API int32 GetPixelSize() const;
	UE_API bool ContainsMips() const;
	UE_API int32 GetNumMipLevels() const;
	UE_API int32 CalculateNumMipLevels(const FIntPoint& NumTiles) const;

	/** Determines if Gpu reader can be utilized. */
	UE_API bool IsOptimizedForGpu() const;

	/** Gets tile dimensions. Returns false if image has no tiles. */
	UE_API bool GetTileSize(FIntPoint& OutTileSize) const;

	UE_API bool HasInputFile() const;

	/**
	 * Get an attribute from the image.
	 *
	 * @param Name		Name of attribute.
	 * @param Value		Will be set to the value of the attribute if the attribute is found.
	 *					Will NOT be set if the attribute is not found.
	 * @return			True if the attribute was found, false otherwise.
	 */
	UE_API bool GetIntAttribute(const FString& Name, int32& Value);

	/**
	 * Get the names of all channels in the file.
	 *
	 * @param OutChannelNames Will contain all channel names (e.g., "R", "G", "B", "Zdepth.Z").
	 */
	UE_API void GetChannelNames(TArray<FString>& OutChannelNames) const;

private:
	TSharedPtr<void> FileContext;
};

/**
 * Base class for our classes that output EXR files.
 */
class FBaseOutputFile
{
public:
	/**
	 * Constructor.
	 *
	 * @param DisplayWindowMin		Normally (0, 0).
	 * @param DisplayWindowMax		Normally (width - 1, height - 1).
	 * @param DataWindowMin			Normally (0, 0).
	 * @param DataWindowMax			Normally (width - 1, height - 1).
	 */
	UE_API FBaseOutputFile(
		const FIntPoint& DisplayWindowMin,
		const FIntPoint& DisplayWindowMax,
		const FIntPoint& DataWindowMin,
		const FIntPoint& DataWindowMax);

	/**
	 * Destructor.
	 */
	UE_API virtual ~FBaseOutputFile();

	/**
	 * Call this to add an attribute to the EXR file.
	 * This MUST be called before CreateOutputFile.
	 *
	 * @param Name		Name for this attribute.
	 * @param Value		Value for this attribute.
	 */
	UE_API void AddIntAttribute(const FString& Name, int32 Value);

	/**
	 * Call this to get the number of mipmap levels.
	 */
	virtual int32 GetNumberOfMipLevels() { return 0; }

protected:
	/** Stores the EXR header object. */
	void* Header;
	/** Stores the EXR object. */
	void* OutputFile;
};

/**
 * Use this to write out tiled EXR images.
 * 
 * Add any attributes after construction.
 * Then you can call CreateOutputFile.
 * After that, you can write out data to the file.
 */
class FTiledRgbaOutputFile : public FBaseOutputFile
{
public:
	/**
	 * Constructor.
	 * 
	 * @param DisplayWindowMin		Normally (0, 0).
	 * @param DisplayWindowMax		Normally (width - 1, height - 1).
	 * @param DataWindowMin			Normally (0, 0).
	 * @param DataWindowMax			Normally (width - 1, height - 1).
	 */
	UE_API FTiledRgbaOutputFile(
		const FIntPoint& DisplayWindowMin,
		const FIntPoint& DisplayWindowMax,
		const FIntPoint& DataWindowMin,
		const FIntPoint& DataWindowMax);

	/**
	 * Call this after adding any attributes BUT before doing anything else.
	 *
	 * @param FilePath			Filename to save to.
	 * @param TileWidth			Width of a tile.
	 * @param TileHeight		Height of a tile.
	 * @param NumChannels		Number of channels out write out.
	 * @param bIsMipsEnabled	True to enable mip mapping.
	 */
	UE_API void CreateOutputFile(const FString& FilePath,
		int32 TileWidth, int32 TileHeight, int32 NumChannels, bool bIsMipsEnabled);

	/**
	 * Call this prior to WriteTile to set where the data is coming from.
	 * 
	 * @param Buffer		Source of data.
	 * @param Stride		A pixels location is calculated by Buffer + x * StrideX + y * StrideY.
	 */
	UE_API void SetFrameBuffer(void* Buffer, const FIntPoint& Stride);

	/**
	 * Call this to write data to a specific tile.
	 *
	 * @param TileX			X coordinate of tile.
	 * @param TileY			Y coordinate of tile.
	 * @param MipLevel		Mipmap level of tile.
	 */
	UE_API void WriteTile(int32 TileX, int32 TileY, int32 MipLevel);

	//~ FBaseOutputFile interface.
	UE_API virtual int32 GetNumberOfMipLevels() override;
};

/**
 * Use this to write out tiled EXR images using the general interface.
 *
 * Add any attributes and AddChannel after construction.
 * Then you can call CreateOutputFile.
 * Then you can call AddFrameBufferChannel for each channel, and then SetFrameBuffer.
 * After that, you can write out data to the file.
 */
class FTiledOutputFile : public FBaseOutputFile
{
public:
	/**
	 * Constructor.
	 *
	 * @param DisplayWindowMin		Normally (0, 0).
	 * @param DisplayWindowMax		Normally (width - 1, height - 1).
	 * @param DataWindowMin			Normally (0, 0).
	 * @param DataWindowMax			Normally (width - 1, height - 1).
	 * @param bInIsTiled			True if you want tiles.
	 */
	UE_API FTiledOutputFile(
		const FIntPoint& DisplayWindowMin,
		const FIntPoint& DisplayWindowMax,
		const FIntPoint& DataWindowMin,
		const FIntPoint& DataWindowMax,
		bool bInIsTiled = true);

	/**
	 * Destructor.
	 */
	UE_API virtual ~FTiledOutputFile();

	/**
	 * Call this before CreateOutputFile for each channel in the image.
	 *
	 * @param Name		Name of channel (e.g. R, G, B, or A). 
	 */
	UE_API void AddChannel(const FString& Name);

	/**
	 * Call this after adding any attributes or channels BUT before doing anything else.
	 *
	 * @param FilePath			Filename to save to.
	 * @param TileWidth			Width of a tile.
	 * @param TileHeight		Height of a tile.
	 * @param bIsMipsEnabled	True to enable mip mapping.
	 * @param NumThreads		Number of threads for EXR to use.
	 */
	UE_API void CreateOutputFile(const FString& FilePath,
		int32 TileWidth, int32 TileHeight, bool bIsMipsEnabled, int32 NumThreads);

	/**
	 * Get the width of a mip level.
	 */
	UE_API int32 GetMipWidth(int32 MipLevel);

	/**
	 * Get the height of a mip level.
	 */
	UE_API int32 GetMipHeight(int32 MipLevel);

	/**
	 * Get number of horizontal tiles for a mip level.
	 */
	UE_API int32 GetNumXTiles(int32 MipLevel);

	/**
	 * Get number of vertical tiles for a mip level.
	 */
	UE_API int32 GetNumYTiles(int32 MipLevel);

	/**
	 * Call this before SetFrameBuffer for each channel in the frame buffer.
	 *
	 * @param Name			Name of channel (e.g. R, G, B, or A).
	 * @param Base			Address of the start of the data.
	 * @param Stride		A pixels location is calculated by Base + x * StrideX + y * StrideY.
	 */
	UE_API void AddFrameBufferChannel(const FString& Name, void* Base,
		const FIntPoint& Stride);

	/**
	 * Call this to change a channel in the frame buffer.
	 *
	 * @param Name			Name of channel (e.g. R, G, B, or A).
	 * @param Base			Address of the start of the data.
	 * @param Stride		A pixels location is calculated by Base + x * StrideX + y * StrideY.
	 */
	UE_API void UpdateFrameBufferChannel(const FString& Name, void* Base,
		const FIntPoint& Stride);

	/**
	 * Call this prior to WriteTile to set where the data is coming from.
	 *
	 * @param Buffer		Source of data.
	 * @param Stride		A pixels location is calculated by Buffer + x * StrideX + y * StrideY.
	 */
	UE_API void SetFrameBuffer();

	/**
	 * Call this to write data to a specific tile.
	 * If tiles are not enabled then this just writes out the whole image.
	 *
	 * @param TileX			X coordinate of tile.
	 * @param TileY			Y coordinate of tile.
	 * @param MipLevel		Mipmap level of tile.
	 */
	UE_API void WriteTile(int32 TileX, int32 TileY, int32 MipLevel);

	/**
	 * Call this to write data to some tiles.
	 * If tiles are not enabled then this just writes out the whole image.
	 *
	 * @param TileX1		X coordinate of first tile.
	 * @param TileX2		X coordinate of last tile.
	 * @param TileY1		Y coordinate of first tile.
	 * @param TileY2		Y coordinate of last tile.
	 * @param MipLevel		Mipmap level of tile.
	 */
	UE_API void WriteTiles(int32 TileX1, int32 TileX2, int32 TileY1, int32 TileY2, int32 MipLevel);

	//~ FBaseOutputFile interface.
	UE_API virtual int32 GetNumberOfMipLevels() override;

private:

	/** Stores the EXR frame buffer object. */
	void* FrameBuffer;
	/** True if tiles are enabled. */
	bool bIsTiled;
};

#undef UE_API
